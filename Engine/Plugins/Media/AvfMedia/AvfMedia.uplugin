{
	"FileVersion" : 3,
	"Version" : 2,
	"VersionName" : "2.0",
	"FriendlyName" : "AVF Media Player",
	"Description" : "Implements a media player using Apple AV Foundation.",
	"Category" : "Media Players",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://forums.unrealengine.com/showthread.php?46879-Media-Framework-Documentation-for-4-5-Preview",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "AvfMedia",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PreLoadingScreen",
			"PlatformAllowList" : [ "IOS", "Mac", "TVOS" ]
		},
		{
			"Name" : "AvfMediaEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "AvfMediaFactory",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "AvfMediaFactory",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PostEngineInit",
			"PlatformAllowList" : [ "IOS", "Linux", "Mac", "TVOS", "Win64" ]
		},
		{
			"Name" : "AvfMediaCapture",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "Default",
			"PlatformAllowList" : [ "IOS", "Mac" ]
		}
	]
}
