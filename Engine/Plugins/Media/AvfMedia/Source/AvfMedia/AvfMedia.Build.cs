// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AvfMedia : ModuleRules
	{
		public AvfMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AvfMediaFactory",
					"Core",
					"Engine",
					"ApplicationCore",
					"MediaUtils",
					"RenderCore",
					"RHI",
					"MetalRHI"
				});
			
			AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
			        
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PublicFrameworks.AddRange(
				new string[] {
					"CoreMedia",
					"CoreVideo",
					"AVFoundation",
					"AudioToolbox",
					"MediaToolbox",
					"QuartzCore"
				});
		}
	}
}
