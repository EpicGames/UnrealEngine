// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvfMediaPrivate.h"

#include "Modules/ModuleManager.h"

#include "Player/AvfMediaPlayer.h"
#include "IAvfMediaModule.h"


DEFINE_LOG_CATEGORY(LogAvfMedia);

#define LOCTEXT_NAMESPACE "FAvfMediaModule"


/**
 * Implements the AVFMedia module.
 */
class FAvfMediaModule
	: public IAvfMediaModule
{
public:

	/** Default constructor. */
	FAvfMediaModule() { }

public:

	//~ IAvfMediaModule interface

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		return MakeShared<FAvfMediaPlayer, ESPMode::ThreadSafe>(EventSink);
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }
};


IMPLEMENT_MODULE(FAvfMediaModule, AvfMedia);


#undef LOCTEXT_NAMESPACE
