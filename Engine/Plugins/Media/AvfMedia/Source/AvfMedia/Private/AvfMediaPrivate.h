// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"


/** Log category for the AvfMedia module. */
DECLARE_LOG_CATEGORY_EXTERN(LogAvfMedia, Log, All);


#if PLATFORM_MAC
	#define AVF_GAME_THREAD_BLOCK 
	#define AVF_GAME_THREAD_RETURN 
	#define AVF_GAME_THREAD_CALL(Block)	GameThreadCall(Block, false)
#else
	#define AVF_GAME_THREAD_BLOCK bool(void)
	#define AVF_GAME_THREAD_RETURN return true
	#define AVF_GAME_THREAD_CALL(Block) \
			[FAppleAsyncTask CreateTaskWithBlock: \
				Block	\
			]
#endif
