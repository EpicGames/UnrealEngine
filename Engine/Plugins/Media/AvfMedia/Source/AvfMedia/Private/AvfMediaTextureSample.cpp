// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvfMediaTextureSample.h"
#include "AvfMediaPrivate.h"

FAvfMediaTextureSample::~FAvfMediaTextureSample()
{
	UnlockAndReleasePixelBuffer();
}

#if WITH_ENGINE

bool FAvfMediaTextureSample::Initialize(
	TRefCountPtr<FRHITexture>& InTexture,
	const FIntPoint& InDim,
	const FIntPoint& InOutputDim,
	FTimespan InTime,
	FTimespan InDuration,
	CVPixelBufferRef InPixelBuffer)
{
	if (!InTexture.IsValid())
	{
		return false;
	}

	Dim = InDim;
	Duration = InDuration;
	OutputDim = InOutputDim;
	Texture = InTexture;
	Time = InTime;

	UnlockAndReleasePixelBuffer();

	if (InPixelBuffer)
	{
		if (CVPixelBufferLockBaseAddress(InPixelBuffer, kCVPixelBufferLock_ReadOnly) == kCVReturnSuccess)
		{
			CFRetain(InPixelBuffer);
			LockedPixelBuffer = InPixelBuffer;
		}
	}

	return true;
}

#else // WITH_ENGINE

bool FAvfMediaTextureSample::Initialize(
	uint8* InBuffer,
	const FIntPoint& InDim,
	const FIntPoint& InOutputDim,
	uint32 InStride,
	FTimespan InTime,
	FTimespan InDuration)
{
	if (InBuffer == nullptr)
	{
		return false;
	}

	const int32 BufferSize = InDim.X * InDim.Y * 4;

	if (BufferSize == 0)
	{
		return false;
	}

	Buffer.Reset(BufferSize);
	Buffer.Append((uint8*)InBuffer, BufferSize);

	Dim = InDim;
	Duration = InDuration;
	OutputDim = InOutputDim;
	Stride = InStride;
	Time = InTime;

	return true;
}

#endif // WITH_ENGINE


const void* FAvfMediaTextureSample::GetBuffer()
{
	return LockedPixelBuffer ? CVPixelBufferGetBaseAddress(LockedPixelBuffer) : nullptr;
}

FIntPoint FAvfMediaTextureSample::GetDim() const
{
	return Dim;
}

FTimespan FAvfMediaTextureSample::GetDuration() const
{
	return Duration;
}

EMediaTextureSampleFormat FAvfMediaTextureSample::GetFormat() const
{
	return EMediaTextureSampleFormat::CharBGRA;
}

FIntPoint FAvfMediaTextureSample::GetOutputDim() const
{
	return OutputDim;
}

uint32 FAvfMediaTextureSample::GetStride() const
{
	return LockedPixelBuffer ? CVPixelBufferGetBytesPerRow(LockedPixelBuffer) : Dim.X * 4;
}

FRHITexture* FAvfMediaTextureSample::GetTexture() const
{
	return Texture;
}

FMediaTimeStamp FAvfMediaTextureSample::GetTime() const
{
	return FMediaTimeStamp(Time);
}

bool FAvfMediaTextureSample::IsCacheable() const
{
	return true;
}

bool FAvfMediaTextureSample::IsOutputSrgb() const
{
	// The GPU texture is created as MTLPixelFormatBGRA8Unorm_sRGB, which tells the RHI
	// to apply sRGB-to-linear conversion on texture read. IsOutputSrgb() is a separate
	// flag consumed by UMediaTexture to determine whether it should ALSO request sRGB
	// conversion when creating render targets from this sample. Returning true here would
	// cause a double conversion (too dark). Return false since the texture format already
	// encodes the sRGB intent at the RHI level.
	return false;
}

void FAvfMediaTextureSample::ShutdownPoolable()
{
	UnlockAndReleasePixelBuffer();
}

void FAvfMediaTextureSample::Reset()
{
	UnlockAndReleasePixelBuffer();
	Texture = nullptr;
}

void FAvfMediaTextureSample::UnlockAndReleasePixelBuffer()
{
	if (LockedPixelBuffer)
	{
		CVPixelBufferUnlockBaseAddress(LockedPixelBuffer, kCVPixelBufferLock_ReadOnly);
		CFRelease(LockedPixelBuffer);
		LockedPixelBuffer = nullptr;
	}
}
