// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaOverlaySample.h"
#include "Internationalization/Text.h"
#include "Misc/Timespan.h"


/**
 * Implements an overlay text sample for AvfMedia.
 */
class FAvfMediaOverlaySample
	: public IMediaOverlaySample
{
public:

	/** Default constructor. */
	FAvfMediaOverlaySample()
		: Time(FTimespan::Zero())
	{ }

	/** Virtual destructor. */
	virtual ~FAvfMediaOverlaySample() = default;

public:

	/**
	 * Initialize the overlay sample.
	 *
	 * @param InString The overlay string.
	 * @param InTime The play time for which the sample was generated.
	 * @return true on success, false otherwise.
	 */
	bool Initialize(const FString& InString, FTimespan InTime)
	{
		Text = FText::FromString(InString);
		Time = InTime;

		return true;
	}

public:

	//~ IMediaOverlaySample interface

	virtual FTimespan GetDuration() const override
	{
		return FTimespan::MaxValue();
	}

	virtual TOptional<FVector2D> GetPosition() const override
	{
		return TOptional<FVector2D>();
	}

	virtual FText GetText() const override
	{
		return Text;
	}

	virtual FMediaTimeStamp GetTime() const override
	{
		return FMediaTimeStamp(Time);
	}

	virtual EMediaOverlaySampleType GetType() const override
	{
		return EMediaOverlaySampleType::Caption;
	}

private:

	/** The overlay text. */
	FText Text;

	/** The play time for which the sample was generated. */
	FTimespan Time;
};
