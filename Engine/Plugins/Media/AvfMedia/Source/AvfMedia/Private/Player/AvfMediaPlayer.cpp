// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvfMediaPlayer.h"
#include "AvfMediaPrivate.h"

#include "HAL/PlatformProcess.h"
#include "IMediaEventSink.h"
#include "MediaSamples.h"
#include "Apple/AppleStringUtils.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/CoreDelegates.h"

#if PLATFORM_MAC
	#include "Mac/CocoaThread.h"
#else
	#include "IOS/IOSAsyncTask.h"
#endif

#include "HAL/FileManager.h"

#include "AvfMediaTracks.h"
#include "Shared/AvfMediaUtils.h"
#include "IMediaAudioSample.h"
#include "HAL/PlatformFileManager.h"
#include "Async/Async.h"

#include "GenericPlatform/GenericPlatformFile.h"
#include "IPlatformFilePak.h"

/* FAVPlayerDelegate
 *****************************************************************************/

/**
 * Cocoa class that can help us with reading player item information.
 */
@interface FAVPlayerDelegate : NSObject
{
};

/** We should only initiate a helper with a media player */
-(FAVPlayerDelegate*) initWithMediaPlayer:(FAvfMediaPlayer*)InPlayer;

/** Destructor */
-(void)dealloc;

/** Notification called when player item reaches the end of playback. */
-(void)playerItemPlaybackEndReached:(NSNotification*)Notification;

/** Reference to the media player which will be responsible for this media session */
@property FAvfMediaPlayer* MediaPlayer;

/** Flag indicating whether the media player item has reached the end of playback */
@property bool bHasPlayerReachedEnd;

@end


@implementation FAVPlayerDelegate
@synthesize MediaPlayer;


-(FAVPlayerDelegate*) initWithMediaPlayer:(FAvfMediaPlayer*)InPlayer
{
	id Self = [super init];
	if (Self)
	{
		MediaPlayer = InPlayer;
	}
	return Self;
}


/** Listener for changes in our media classes properties. */
- (void) observeValueForKeyPath:(NSString*)keyPath
		ofObject:	(id)object
		change:		(NSDictionary*)change
		context:	(void*)context
{
	if ([keyPath isEqualToString:@"status"])
	{
		if (object == (id)context)
		{
			MediaPlayer->OnStatusNotification();
		}
	}
}


- (void)dealloc
{
	[super dealloc];
}


-(void)playerItemPlaybackEndReached:(NSNotification*)Notification
{
	MediaPlayer->OnEndReached();
}

@end

/* Media Resource Data loader, e.g for Pak files
 *****************************************************************************/
@interface FAVMediaAssetResourceLoaderDelegate : NSObject <AVAssetResourceLoaderDelegate>
{
	@public FString Path;
	@public TSharedPtr<FArchive, ESPMode::ThreadSafe> FileAReader;
	@public FCriticalSection CriticalSection;
	@public bool bInitialized;
	@public TFuture<void> AsyncLoadingTask;
}
@end

@implementation FAVMediaAssetResourceLoaderDelegate

-(FAVMediaAssetResourceLoaderDelegate*) initWithPath:(FString&)InPath
{
	self = [super init];
	if (self != nil)
	{
		bInitialized = false;
		Path = InPath;
	}
	return self;
}

- (void) dealloc
{
	{
		FScopeLock ScopeLock(&CriticalSection);

		if(FileAReader.IsValid())
		{
			FileAReader->Close();
			FileAReader = nullptr;
		}
	}

	[super dealloc];
}

- (BOOL)resourceLoader:(AVAssetResourceLoader *)resourceLoader shouldWaitForLoadingOfRequestedResource:(AVAssetResourceLoadingRequest *)loadingRequest
{
	// There should be no need to queue these up - if it turns out we need to do that - then add an ordered queue of loadingRequest objects
	[loadingRequest retain];

	// Allow this function to return quickly so the resource loader knows the data is probabky coming and doesn't error
	AsyncLoadingTask = Async(EAsyncExecution::ThreadPool, [self, loadingRequest]()
	{
		FScopeLock ScopeLock(&CriticalSection);

		// If the file reader is created on the Apple callback queue then the PakLoader will throw thread errors
		if (!bInitialized)
		{
			FileAReader = MakeShareable( IFileManager::Get().CreateFileReader(*Path) );
			bInitialized = true;
		}

		if(FileAReader.IsValid() && !FileAReader->IsError())
		{
			// Fill out content information request - if required
			if(loadingRequest.contentInformationRequest)
			{
				// See https://developer.apple.com/library/archive/documentation/Miscellaneous/Reference/UTIRef/Articles/System-DeclaredUniformTypeIdentifiers.html
				// And loadingRequest.contentInformationRequest.allowedContentTypes;
				loadingRequest.contentInformationRequest.contentType = @"public.mpeg-4";
				loadingRequest.contentInformationRequest.byteRangeAccessSupported = YES;
				loadingRequest.contentInformationRequest.contentLength = FileAReader->TotalSize();
			}

			// Fetch data from file - if required
			if(loadingRequest.dataRequest)
			{
				int64 Offset = loadingRequest.dataRequest.requestedOffset;
				int64 ByteCount = loadingRequest.dataRequest.requestedLength;

				check(Offset >= 0);
				check(ByteCount > 0);

				if(Offset + ByteCount <= FileAReader->TotalSize())
				{
					FileAReader->Seek(Offset);

					// Don't read the whole requested data range at once - the resource loader often asks for very large data sizes
					// If we feed it (using respondWithData:) in chunks, it decides it has had enough data usually after a few MB,
					// then it marks the request as cancelled, this is not an error, before issuing a different request at some point later.
					// This keeps our peak memory usage down and limits the amount of data we are serializing.

					const int64 MaxChunkBytes = 1024 * 1024 * 1; // in single MB chunks
					while(ByteCount > 0 && !loadingRequest.isCancelled && !FileAReader->IsError())
					{
						int64 ChunkByteCount = MIN(MaxChunkBytes, ByteCount);
						ByteCount -= ChunkByteCount;
						check(ByteCount >= 0);

						NSMutableData* nsLoadedData = [[NSMutableData alloc] initWithLength:ChunkByteCount];
						uint8* pMemory = (uint8*)nsLoadedData.mutableBytes;
						check(pMemory);

						FileAReader->Serialize(pMemory, ChunkByteCount);
						@autoreleasepool{
							[loadingRequest.dataRequest respondWithData:nsLoadedData];
						}
						[nsLoadedData release];
					}
				}
			}
		}

		// Check file reader is not in error state after potential seek and data read operations
		if(FileAReader.IsValid() && !FileAReader->IsError())
		{
			@autoreleasepool{
				[loadingRequest finishLoading];
			}
		}
		else
		{
			[loadingRequest finishLoadingWithError:nil];
		}

		[loadingRequest release];
	});

	return YES;
}

- (BOOL)resourceLoader:(AVAssetResourceLoader *)resourceLoader shouldWaitForRenewalOfRequestedResource:(AVAssetResourceRenewalRequest *)renewalRequest
{
	// Don't set contentInformationRequest.renewalDate and we should not have to handle this case
	return NO;
}

- (void)resourceLoader:(AVAssetResourceLoader *)resourceLoader didCancelLoadingRequest:(AVAssetResourceLoadingRequest *)loadingRequest
{

}

- (BOOL)resourceLoader:(AVAssetResourceLoader *)resourceLoader shouldWaitForResponseToAuthenticationChallenge:(NSURLAuthenticationChallenge *)authenticationChallenge
{
	return NO;
}

- (void)resourceLoader:(AVAssetResourceLoader *)resourceLoader didCancelAuthenticationChallenge:(NSURLAuthenticationChallenge *)authenticationChallenge
{

}
@end

/* Sync Control Class for consumed samples
 *****************************************************************************/
class FAvfMediaSamples : public FMediaSamples
{
public:
	FAvfMediaSamples()
	: FMediaSamples()
	, AudioSyncSampleTime(FTimespan::MinValue())
	, VideoSyncSampleTime(FTimespan::MinValue())
	{}

	virtual ~FAvfMediaSamples()
	{}

	virtual bool FetchAudio(TRange<FTimespan> TimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample) override
	{
		bool bResult = FMediaSamples::FetchAudio(TimeRange, OutSample);

		if(FTimespan::MinValue() == AudioSyncSampleTime && bResult && OutSample.IsValid())
		{
			AudioSyncSampleTime = OutSample->GetTime().Time + OutSample->GetDuration();
		}

		return bResult;
	}

	virtual bool FetchVideo(TRange<FTimespan> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample)
	{
		bool bResult = FMediaSamples::FetchVideo(TimeRange, OutSample);

		if(FTimespan::MinValue() == VideoSyncSampleTime && bResult && OutSample.IsValid())
		{
			VideoSyncSampleTime = OutSample->GetTime().Time + OutSample->GetDuration();
		}

		return bResult;
	}

	void ClearSyncSampleTimes ()			 { AudioSyncSampleTime = VideoSyncSampleTime = FTimespan::MinValue(); }
	FTimespan GetAudioSyncSampleTime() const { return AudioSyncSampleTime; }
	FTimespan GetVideoSyncSampleTime() const { return VideoSyncSampleTime; }

private:

	TAtomic<FTimespan> AudioSyncSampleTime;
	TAtomic<FTimespan> VideoSyncSampleTime;
};

/* FAvfMediaPlayer structors
 *****************************************************************************/

FAvfMediaPlayer::FAvfMediaPlayer(IMediaEventSink& InEventSink)
	: EventSink(InEventSink)
{
	CurrentRate = 0.0f;
	CurrentState = EMediaState::Closed;
    CurrentTime = FTimespan::Zero();

	Duration = FTimespan::Zero();
	MediaUrl = FString();
	ShouldLoop = false;

	MediaHelper = nil;
    MediaPlayer = nil;
	PlayerItem = nil;
	MediaResourceLoader = nil;

	bPrerolled = false;
	bTimeSynced = false;
	bSeeking = false;

	Samples = new FAvfMediaSamples;
	Tracks = new FAvfMediaTracks(*Samples);
}


FAvfMediaPlayer::~FAvfMediaPlayer()
{
	Close();

	delete Tracks;
	Tracks = nullptr;

	delete Samples;
	Samples = nullptr;
}

/* FAvfMediaPlayer Sample Sync helpers
 *****************************************************************************/
void FAvfMediaPlayer::ClearTimeSync()
{
	bTimeSynced = false;
	if(Samples != nullptr)
	{
		Samples->ClearSyncSampleTimes();
	}
}

FTimespan FAvfMediaPlayer::GetAudioTimeSync() const
{
	FTimespan Sync = FTimespan::MinValue();
	if(Samples != nullptr)
	{
		Sync = Samples->GetAudioSyncSampleTime();
	}
	return Sync;
}

FTimespan FAvfMediaPlayer::GetVideoTimeSync() const
{
	FTimespan Sync = FTimespan::MinValue();
	if(Samples != nullptr)
	{
		Sync = Samples->GetVideoSyncSampleTime();
	}
	return Sync;
}


/* FAvfMediaPlayer interface
 *****************************************************************************/

void FAvfMediaPlayer::OnEndReached()
{
	if (ShouldLoop)
	{
		PlayerTasks.Enqueue([this]()
		{
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackEndReached);
			Seek(CurrentRate < 0.f ? Duration : FTimespan::Zero());
		});
	}
	else
	{
		CurrentState = EMediaState::Paused;
		CurrentRate = 0.0f;

		PlayerTasks.Enqueue([this]()
		{
			Seek(FTimespan::Zero());
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackEndReached);
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);
		});
	}
}


void FAvfMediaPlayer::OnStatusNotification()
{
	PlayerTasks.Enqueue([this]()
	{
		switch(PlayerItem.status)
		{
			case AVPlayerItemStatusReadyToPlay:
			{
				if (Duration == FTimespan::Zero() || CurrentState == EMediaState::Closed)
				{
					Tracks->Initialize(PlayerItem, Info);
					EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);

					Duration = FTimespan::FromSeconds(CMTimeGetSeconds(PlayerItem.asset.duration));
					CurrentState = (CurrentState == EMediaState::Closed) ? EMediaState::Stopped : CurrentState;

					if (!bPrerolled)
					{
						if(MediaResourceLoader != nil)
						{
							// If there is a resource loader - don't preroll
							bPrerolled = true;
							CurrentState = EMediaState::Stopped;
							EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
						}
						else
						{
							// Preroll for playback.
							[MediaPlayer prerollAtRate:1.0f completionHandler:^(BOOL bFinished)
							{
								// For some reason we now get bFinished==NO most of the time for
								// unknown reasons. According to the documentation failure is caused
								// if the playback rate or time is changed during prerolling, but we
								// do neither, so the cause of the failure is unknown.
							#if 0
								if (bFinished)
								{
									PlayerTasks.Enqueue([this]()
									{
										if(PlayerItem.status == AVPlayerItemStatusReadyToPlay)
										{
											bPrerolled = true;
											CurrentState = EMediaState::Stopped;
											EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
										}
									});
								}
								else
								{
									PlayerTasks.Enqueue([this]()
									{
										CurrentState = EMediaState::Error;
										EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
									});
								}
							#else
								PlayerTasks.Enqueue([this]()
								{
									if(PlayerItem.status == AVPlayerItemStatusReadyToPlay)
									{
										bPrerolled = true;
										CurrentState = EMediaState::Stopped;
										EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
									}
									else
									{
										CurrentState = EMediaState::Error;
										EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
									}
								});
							#endif
							}];
						}
					}
				}

				break;
			}
			case AVPlayerItemStatusFailed:
			{
				if (Duration == FTimespan::Zero() || CurrentState == EMediaState::Closed)
				{
					CurrentState = EMediaState::Error;
					EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
				}
				else
				{
					CurrentState = EMediaState::Error;
					EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);
				}
				break;
			}
			case AVPlayerItemStatusUnknown:
			default:
			{
				break;
			}
		}
	});
}


/* IMediaPlayer interface
 *****************************************************************************/

void FAvfMediaPlayer::Close()
{
	if (CurrentState == EMediaState::Closed)
	{
		return;
	}

    if (EnteredForegroundHandle.IsValid())
    {
        FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(EnteredForegroundHandle);
        EnteredForegroundHandle.Reset();
    }

    if (HasReactivatedHandle.IsValid())
    {
        FCoreDelegates::ApplicationHasReactivatedDelegate.Remove(HasReactivatedHandle);
        HasReactivatedHandle.Reset();
    }

    if (EnteredBackgroundHandle.IsValid())
    {
        FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(EnteredBackgroundHandle);
        EnteredBackgroundHandle.Reset();
    }

    if (WillDeactivateHandle.IsValid())
    {
        FCoreDelegates::ApplicationWillDeactivateDelegate.Remove(WillDeactivateHandle);
        WillDeactivateHandle.Reset();
    }

	if (AudioRouteChangedHandle.IsValid())
	{
		FCoreDelegates::AudioRouteChangedDelegate.Remove(AudioRouteChangedHandle);
		AudioRouteChangedHandle.Reset();
	}

	CurrentTime = FTimespan::Zero();
	MediaUrl = FString();

	if (PlayerItem != nil)
	{
		if (MediaHelper != nil)
		{
			[[NSNotificationCenter defaultCenter] removeObserver:MediaHelper name:AVPlayerItemDidPlayToEndTimeNotification object:PlayerItem];
			[PlayerItem removeObserver:MediaHelper forKeyPath:@"status"];
		}

		// Cancel the pending seeks before releasing the MediaPlayer
		[PlayerItem cancelPendingSeeks];
		[PlayerItem.asset cancelLoading];
		// Override the completionHandler in case it hasn't been invoked before releasing PlayerItem.
		[PlayerItem.asset loadValuesAsynchronouslyForKeys:@[@"tracks"] completionHandler:^
		{
		}];
		MediaPlayer.rate = 0.0;
		[PlayerItem release];
		PlayerItem = nil;
	}

	if (MediaHelper != nil)
	{
		[MediaHelper release];
		MediaHelper = nil;
	}

	if (MediaPlayer != nil)
	{
		// If we don't remove the current player item then the retain count is > 1 for the MediaPlayer then on it's release then the MetalPlayer stays around forever
		[MediaPlayer replaceCurrentItemWithPlayerItem:nil];
		[MediaPlayer release];
		MediaPlayer = nil;
	}

	if(MediaResourceLoader != nil)
	{
		// Wait the loading task to finish before releasing the loader.
		MediaResourceLoader->AsyncLoadingTask.Wait();
		[MediaResourceLoader release];
		MediaResourceLoader = nil;
	}

	Tracks->Reset();
	EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);

	CurrentState = EMediaState::Closed;
	Duration = CurrentTime = FTimespan::Zero();
	Info.Empty();

	EventSink.ReceiveMediaEvent(EMediaEvent::MediaClosed);

	bPrerolled = false;
	bSeeking = false;

	CurrentRate = 0.f;

	ClearTimeSync();
}


IMediaCache& FAvfMediaPlayer::GetCache()
{
	return *this;
}


IMediaControls& FAvfMediaPlayer::GetControls()
{
	return *this;
}


FString FAvfMediaPlayer::GetInfo() const
{
	return Info;
}


FGuid FAvfMediaPlayer::GetPlayerPluginGUID() const
{
	static FGuid PlayerPluginGUID(0xef660169, 0xa304472b, 0xb69e1281, 0xc1687654);
	return PlayerPluginGUID;
}


IMediaSamples& FAvfMediaPlayer::GetSamples()
{
	return *Samples;
}


FString FAvfMediaPlayer::GetStats() const
{
	FString Result;

	Tracks->AppendStats(Result);

	return Result;
}


IMediaTracks& FAvfMediaPlayer::GetTracks()
{
	return *Tracks;
}


FString FAvfMediaPlayer::GetUrl() const
{
	return MediaUrl;
}


IMediaView& FAvfMediaPlayer::GetView()
{
	return *this;
}

bool FAvfMediaPlayer::Open(const FString& Url, const IMediaOptions* /*Options*/)
{
	Close();

	NSURL* nsMediaUrl = nil;
	FString Path;

	bool bPakResourceLoading = false;

	if (Url.StartsWith(TEXT("file://")))
	{
		// Media Framework doesn't percent encode the URL, so the path portion is just a native file path.
		// Extract it and then use it create a proper URL.
		Path = Url.Mid(7);
		nsMediaUrl = [NSURL fileURLWithPath:Path.GetNSString() isDirectory:NO];

#if !WITH_EDITOR
		// Check if this file is in a pak file - if it is, AV Player can't load this we need to handle the data load requests manually
		FPakPlatformFile* PakFileMgr = (FPakPlatformFile*)(FPlatformFileManager::Get().FindPlatformFile(FPakPlatformFile::GetTypeName()));
		if (PakFileMgr != nullptr && PakFileMgr->FindFileInPakFiles(*Path, nullptr, nullptr))
		{
			// Force the AV player to not be able to decode the scheme - this makes it use our ResourceLoader
			NSString* formatString = [NSString stringWithFormat:@"UE-Media://%@", [Path.GetNSString() stringByAddingPercentEncodingWithAllowedCharacters:[NSCharacterSet URLPathAllowedCharacterSet]]];
			nsMediaUrl = [NSURL URLWithString:formatString];

			bPakResourceLoading = true;
		}
#endif
	}
	else
	{
		// Assume that this has been percent encoded for now - when we support HTTP Live Streaming we will need to check for that.
		nsMediaUrl = [NSURL URLWithString: Url.GetNSString()];
	}

	// open media file
	if (nsMediaUrl == nil)
	{
		UE_LOGF(LogAvfMedia, Error, "Failed to open Media file: %ls", *Url);

		return false;
	}

	// On non-Mac Apple OSes the path is:
	//	a) case-sensitive
	//	b) relative to the 'cookeddata' directory, not the notional GameContentDirectory which is 'virtual' and resolved by the FIOSPlatformFile calls
#if !PLATFORM_MAC
	if ([[nsMediaUrl scheme] isEqualToString:@"file"])
	{
		FString FullPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*Path);
		nsMediaUrl = [NSURL fileURLWithPath: FullPath.GetNSString() isDirectory:NO];
	}
#endif

	// create player instance
	MediaUrl = FPaths::GetCleanFilename(Url);
	MediaPlayer = [[AVPlayer alloc] init];

	if (!MediaPlayer)
	{
		UE_LOGF(LogAvfMedia, Error, "Failed to create instance of an AVPlayer");
		return false;
	}

	MediaPlayer.actionAtItemEnd = AVPlayerActionAtItemEndPause;

	// create player item
	MediaHelper = [[FAVPlayerDelegate alloc] initWithMediaPlayer:this];
	check(MediaHelper != nil);

	// Use URL asset which gives us resource loading ability if system can't handle the scheme
	AVURLAsset* urlAsset = [[AVURLAsset alloc] initWithURL:nsMediaUrl options:nil];

	if(bPakResourceLoading)
	{
		MediaResourceLoader = [[FAVMediaAssetResourceLoaderDelegate alloc] initWithPath:Path];
		[urlAsset.resourceLoader setDelegate:MediaResourceLoader queue:dispatch_get_global_queue(QOS_CLASS_DEFAULT, 0)];
	}

	PlayerItem = [[AVPlayerItem playerItemWithAsset:urlAsset] retain];
	[urlAsset release];

	if (PlayerItem == nil)
	{
		UE_LOGF(LogAvfMedia, Error, "Failed to open player item with Url: %ls", *Url);
		return false;
	}

	CurrentState = EMediaState::Preparing;

	// load tracks
	[[PlayerItem asset] loadValuesAsynchronouslyForKeys:@[@"tracks"] completionHandler:^
	{
		NSError* Error = nil;

		if ([[PlayerItem asset] statusOfValueForKey:@"tracks" error : &Error] == AVKeyValueStatusLoaded)
		{
			// File movies will be ready now
			if (PlayerItem.status == AVPlayerItemStatusReadyToPlay)
			{
				PlayerTasks.Enqueue([this]()
				{
					OnStatusNotification();
				});
			}
		}
		else if (Error != nullptr)
		{
			NSDictionary *userInfo = [Error userInfo];
			NSString *errstr = [[userInfo objectForKey : NSUnderlyingErrorKey] localizedDescription];

			UE_LOGF(LogAvfMedia, Warning, "Failed to load video tracks. [%ls]", *FString(errstr));

			PlayerTasks.Enqueue([this]()
			{
				CurrentState = EMediaState::Error;
				EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
			});
		}
	}];

	[[NSNotificationCenter defaultCenter] addObserver:MediaHelper selector:@selector(playerItemPlaybackEndReached:) name:AVPlayerItemDidPlayToEndTimeNotification object:PlayerItem];
	[PlayerItem addObserver:MediaHelper forKeyPath:@"status" options:0 context:PlayerItem];

	MediaPlayer.rate = 0.0;
	CurrentTime = FTimespan::Zero();

	[MediaPlayer replaceCurrentItemWithPlayerItem : PlayerItem];

	if (!EnteredForegroundHandle.IsValid())
    {
        EnteredForegroundHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FAvfMediaPlayer::HandleApplicationHasEnteredForeground);
    }
    if (!HasReactivatedHandle.IsValid())
    {
        HasReactivatedHandle = FCoreDelegates::ApplicationHasReactivatedDelegate.AddRaw(this, &FAvfMediaPlayer::HandleApplicationActivate);
    }

    if (!EnteredBackgroundHandle.IsValid())
    {
        EnteredBackgroundHandle = FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FAvfMediaPlayer::HandleApplicationWillEnterBackground);
    }
    if (!WillDeactivateHandle.IsValid())
    {
        WillDeactivateHandle = FCoreDelegates::ApplicationWillDeactivateDelegate.AddRaw(this, &FAvfMediaPlayer::HandleApplicationDeactivate);
    }

	if (!AudioRouteChangedHandle.IsValid())
	{
		AudioRouteChangedHandle = FCoreDelegates::AudioRouteChangedDelegate.AddRaw(this, &FAvfMediaPlayer::HandleAudioRouteChanged);
	}

	return true;
}


bool FAvfMediaPlayer::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& /*Archive*/, const FString& /*OriginalUrl*/, const IMediaOptions* /*Options*/)
{
	return false; // not supported
}


void FAvfMediaPlayer::TickAudio()
{
	// NOP
}


void FAvfMediaPlayer::TickFetch(FTimespan DeltaTime, FTimespan /*Timecode*/)
{
	if ((CurrentState > EMediaState::Error) && (Duration > FTimespan::Zero()))
	{
		Tracks->ProcessVideo();
	}
}


void FAvfMediaPlayer::TickInput(FTimespan DeltaTime, FTimespan /*Timecode*/)
{
	if ((CurrentState > EMediaState::Error) && (Duration > FTimespan::Zero()))
	{
		switch(CurrentState)
		{
			case EMediaState::Playing:
			{
				if(bSeeking)
				{
					ClearTimeSync();
				}
				else
				{
					if(!bTimeSynced)
					{
						FTimespan SyncTime = FTimespan::MinValue();
#if AUDIO_PLAYBACK_VIA_ENGINE
						// There is no audio in reverse - can't use as sync point - same issue as if no audio track video
						if(Tracks->GetSelectedTrack(EMediaTrackType::Audio) != INDEX_NONE && CurrentRate >= 0.f)
						{
							SyncTime = GetAudioTimeSync();
						}
						else /* Default Use AVPlayer time*/
#endif
						{
							SyncTime = FTimespan::FromSeconds(CMTimeGetSeconds(MediaPlayer.currentTime));
						}

						if(SyncTime != FTimespan::MinValue())
						{
							bTimeSynced = true;
							CurrentTime = SyncTime;
						}
					}
					else
					{
						// On certain movies (short ones it seems) the AVPlayerItemDidPlayToEndTimeNotification can be delayed - clamp the time to the duration
						CurrentTime = FMath::Clamp(CurrentTime + (DeltaTime * CurrentRate), FTimespan::Zero(), Duration);
					}
				}
				break;
			}
			case EMediaState::Stopped:
			case EMediaState::Closed:
			case EMediaState::Error:
			case EMediaState::Preparing:
			{
				CurrentTime = FTimespan::Zero();
				break;
			}
			case EMediaState::Paused:
			default:
			{
				break;
			}
		}
	}

	// process deferred tasks
	TFunction<void()> Task;

	while (PlayerTasks.Dequeue(Task))
	{
		Task();
	}
}


/* IMediaControls interface
 *****************************************************************************/

bool FAvfMediaPlayer::CanControl(EMediaControl Control) const
{
	if (!bPrerolled)
	{
		return false;
	}

	if (Control == EMediaControl::Pause)
	{
		return (CurrentState == EMediaState::Playing);
	}

	if (Control == EMediaControl::Resume)
	{
		return (CurrentState != EMediaState::Playing);
	}

	if ((Control == EMediaControl::Scrub) || (Control == EMediaControl::Seek))
	{
		return true;
	}

	return false;
}


FTimespan FAvfMediaPlayer::GetDuration() const
{
	return Duration;
}


float FAvfMediaPlayer::GetRate() const
{
	return CurrentRate;
}


EMediaState FAvfMediaPlayer::GetState() const
{
	return CurrentState;
}


EMediaStatus FAvfMediaPlayer::GetStatus() const
{
	return EMediaStatus::None;
}


TRangeSet<float> FAvfMediaPlayer::GetSupportedRates(EMediaRateThinning Thinning) const
{
	TRangeSet<float> Result;

	Result.Add(TRange<float>(PlayerItem.canPlayFastReverse ? -8.0f : -1.0f, 0.0f));
	Result.Add(TRange<float>(0.0f, PlayerItem.canPlayFastForward ? 8.0f : 0.0f));

	return Result;
}


FTimespan FAvfMediaPlayer::GetTime() const
{
	return CurrentTime;
}


bool FAvfMediaPlayer::IsLooping() const
{
	return ShouldLoop;
}

bool FAvfMediaPlayer::Seek(const FTimespan& Time)
{
	if (bPrerolled)
	{
		bSeeking = true;
		ClearTimeSync();

		CurrentTime = Time;

		double TotalSeconds = Time.GetTotalSeconds();
		CMTime CurrentTimeInSeconds = CMTimeMakeWithSeconds(TotalSeconds, 1000);

		static CMTime Tolerance = CMTimeMakeWithSeconds(0.01, 1000);
		[MediaPlayer seekToTime:CurrentTimeInSeconds toleranceBefore:Tolerance toleranceAfter:Tolerance completionHandler:^(BOOL bFinished)
		{
			if(bFinished)
			{
				PlayerTasks.Enqueue([this]()
				{
					bSeeking = false;
					EventSink.ReceiveMediaEvent(EMediaEvent::SeekCompleted);
				});
			}
		}];
	}

	return true;
}


bool FAvfMediaPlayer::SetLooping(bool Looping)
{
	ShouldLoop = Looping;

	if (ShouldLoop)
	{
		MediaPlayer.actionAtItemEnd = AVPlayerActionAtItemEndNone;
	}
	else
	{
		MediaPlayer.actionAtItemEnd = AVPlayerActionAtItemEndPause;
	}

	return true;
}


bool FAvfMediaPlayer::SetRate(float Rate)
{
	CurrentRate = Rate;

	if (bPrerolled)
	{
		[MediaPlayer setRate : CurrentRate];

		if (FMath::IsNearlyZero(CurrentRate) && CurrentState != EMediaState::Paused)
		{
			CurrentState = EMediaState::Paused;
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);
		}
		else
		{
			if(CurrentState != EMediaState::Playing)
			{
				ClearTimeSync();

				CurrentState = EMediaState::Playing;
				EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);
			}
		}

		// Use AVPlayer Mute to control reverse playback audio playback
		// Only needed if !AUDIO_PLAYBACK_VIA_ENGINE - however - keep all platforms the same
		bool bMuteAudio = Rate < 0.f;
		if(bMuteAudio)
		{
			MediaPlayer.muted = YES;
		}
		else
		{
			MediaPlayer.muted = NO;
		}

#if AUDIO_PLAYBACK_VIA_ENGINE
		Tracks->ApplyMuteState(bMuteAudio);
#endif
	}

	return true;
}


#if PLATFORM_IOS || PLATFORM_TVOS
bool FAvfMediaPlayer::SetNativeVolume(float Volume)
{
	if (MediaPlayer != nil)
	{
		MediaPlayer.volume = Volume < 0.0f ? 0.0f : (Volume < 1.0f ? Volume : 1.0f);
		return true;
	}
	return false;
}
#endif


void FAvfMediaPlayer::HandleApplicationHasEnteredForeground()
{
    // check the state to ensure we are still playing
    if ((CurrentState == EMediaState::Playing) && MediaPlayer != nil)
    {
        [MediaPlayer play];
    }
}

void FAvfMediaPlayer::HandleApplicationWillEnterBackground()
{
    // check the state to ensure we are still playing
    if ((CurrentState == EMediaState::Playing) && MediaPlayer != nil)
    {
        [MediaPlayer pause];
    }
}

void FAvfMediaPlayer::HandleApplicationActivate()
{
	// check the state to ensure we are still playing
	if ((CurrentState == EMediaState::Playing) && MediaPlayer != nil)
	{
		[MediaPlayer play];
	}
}

void FAvfMediaPlayer::HandleApplicationDeactivate()
{
	// check the state to ensure we are still playing
	if ((CurrentState == EMediaState::Playing) && MediaPlayer != nil)
	{
		[MediaPlayer pause];
	}
}

void FAvfMediaPlayer::HandleAudioRouteChanged(bool InDeviceAvailable)
{
	if ((CurrentState == EMediaState::Playing) && MediaPlayer != nil)
	{
		if (!InDeviceAvailable)
		{
			// restart the media - route it to the active audio device
			// i.e. when unplugging the headphones
			[MediaPlayer pause];

			[MediaPlayer play];
		}
	}
}
