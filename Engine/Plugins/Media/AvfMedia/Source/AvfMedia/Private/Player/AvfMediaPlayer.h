// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "IMediaCache.h"
#include "IMediaControls.h"
#include "IMediaPlayer.h"
#include "IMediaView.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import <AVFoundation/AVFoundation.h>

class FAvfMediaTracks;
class FAvfMediaSamples;
class IMediaEventSink;

@class AVPlayer;
@class AVPlayerItem;
@class FAVPlayerDelegate;
@class FAVMediaAssetResourceLoaderDelegate;


/**
 * Implements a media player using the AV framework.
 */
class FAvfMediaPlayer
	: public IMediaPlayer
	, protected IMediaCache
	, protected IMediaControls
	, protected IMediaView
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InEventSink The object that receives media events from this player.
	 */
	FAvfMediaPlayer(IMediaEventSink& InEventSink);

	/** Destructor. */
	~FAvfMediaPlayer();

public:

	/** Called by the delegate when the playback reaches the end. */
	void OnEndReached();

	/** Called by the delegate whenever the player item status changes. */
	void OnStatusNotification();

public:

	//~ IMediaPlayer interface

	virtual void Close() override;
	virtual IMediaCache& GetCache() override;
	virtual IMediaControls& GetControls() override;
	virtual FString GetInfo() const override;
	virtual FGuid GetPlayerPluginGUID() const override;
	virtual IMediaSamples& GetSamples() override;
	virtual FString GetStats() const override;
	virtual IMediaTracks& GetTracks() override;
	virtual FString GetUrl() const override;
	virtual IMediaView& GetView() override;
	virtual bool Open(const FString& Url, const IMediaOptions* Options) override;
	virtual bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options) override;
	virtual void TickAudio() override;
	virtual void TickFetch(FTimespan DeltaTime, FTimespan Timecode) override;
	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override;

protected:

	//~ IMediaControls interface

	virtual bool CanControl(EMediaControl Control) const override;
	virtual FTimespan GetDuration() const override;
	virtual float GetRate() const override;
	virtual EMediaState GetState() const override;
	virtual EMediaStatus GetStatus() const override;
	virtual TRangeSet<float> GetSupportedRates(EMediaRateThinning Thinning) const override;
	virtual FTimespan GetTime() const override;
	virtual bool IsLooping() const override;
	virtual bool Seek(const FTimespan& Time) override;
	virtual bool SetLooping(bool Looping) override;
	virtual bool SetRate(float Rate) override;
#if PLATFORM_IOS || PLATFORM_TVOS
	virtual bool SetNativeVolume(float Volume) override;
#endif

private:
    /**  Callback for when the application is resumed in the foreground */
    void HandleApplicationHasEnteredForeground();
    
    /** Callback for when the applicaiton is being paused in the background */
    void HandleApplicationWillEnterBackground();

	/** Callback for when the application is moved from the inactive to active state */
	void HandleApplicationActivate();

	/** Callback for when the application is moved from the active to inactive state */
	void HandleApplicationDeactivate();

	/** Callback for when the audio device is changed 
	* @param InDeviceAvailable - true if the device is available
	*/
	void HandleAudioRouteChanged(bool InDeviceAvailable);

	/** Clears the Time Sync flag*/
	void ClearTimeSync();
	
	/** Returns the consumed buffer type sync Points */
	FTimespan GetAudioTimeSync() const;
	FTimespan GetVideoTimeSync() const;
	
	/** The current playback rate. */
	float CurrentRate;

	/** Media playback state. */
	EMediaState CurrentState;

	/** The current time of the playback. */
	FTimespan CurrentTime;

	/** The duration of the media. */
    FTimespan Duration;

	/** The media event handler. */
	IMediaEventSink& EventSink;

	/** Media information string. */
	FString Info;

	/** Cocoa helper object we can use to keep track of ns property changes in our media items */
	FAVPlayerDelegate* MediaHelper;
	
	/** Cocoa Media helper object for Pak file loading */
	FAVMediaAssetResourceLoaderDelegate* MediaResourceLoader;
	
	/** The AVFoundation media player */
	AVPlayer* MediaPlayer;

	/** The URL of the currently opened media. */
	FString MediaUrl;

	/** The player item which the media player uses to progress. */
	AVPlayerItem* PlayerItem;

	/** Tasks to be executed on the player thread. */
	TQueue<TFunction<void()>> PlayerTasks;

	/** The media sample queue. */
	FAvfMediaSamples* Samples;

	/** Should the video loop to the beginning at completion */
    bool ShouldLoop;

	/** The media track collection. */
	FAvfMediaTracks* Tracks;
	
	/** Playback primed and ready when set */
	bool bPrerolled;
	
	/** Media Player is currently seeking */
	bool bSeeking;
	
	/** Set false until the first audio (or video if none) sample has been consumed after seeking or prerolling or, on non Engine mixer platforms first tick after seek */
	bool bTimeSynced;

	/** Mutex to ensure thread-safe access */
	FCriticalSection CriticalSection;
    
    /** Foreground/background delegate for resume */
    FDelegateHandle EnteredForegroundHandle;
    FDelegateHandle HasReactivatedHandle;

    /** Foreground/background delegate for pause */
    FDelegateHandle EnteredBackgroundHandle;
    FDelegateHandle WillDeactivateHandle;

	/** Delegate for changing the audio device  */
	FDelegateHandle AudioRouteChangedHandle;

};
