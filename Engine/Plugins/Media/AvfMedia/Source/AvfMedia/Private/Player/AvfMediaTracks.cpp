// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvfMediaTracks.h"

#if PLATFORM_MAC
#include "Apple/ScopeAutoreleasePool.h"
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "Apple/ScopeAutoreleasePool.h"
#include "IOS/IOSSystemIncludes.h"
#endif
#include "Misc/AssertionMacros.h"

#include "AvfMediaPrivate.h"

#include "Containers/ResourceArray.h"
#include "MediaSamples.h"
#include "Misc/ScopeLock.h"

#if WITH_ENGINE
	#include "RenderingThread.h"
#endif

#include "AvfMediaAudioSample.h"
#include "AvfMediaOverlaySample.h"
#include "AvfMediaVideoSampler.h"
#include "Shared/AvfMediaUtils.h"

#import <AudioToolbox/AudioToolbox.h>

NS_ASSUME_NONNULL_BEGIN

static int32 AVFMediaForceDecodeBGRA = 1;
FAutoConsoleVariableRef CVarAVFMediaForceDecodeBGRA(
	TEXT("m.avf.ForceDecodeBGRA"),
	AVFMediaForceDecodeBGRA,
	TEXT("Selects how H.264 file playback is decoded on Apple platforms. Other codecs and live capture\n")
	TEXT("always decode to BGRA regardless of this setting.\n")
	TEXT("1: Decode directly to BGRA via AVFoundation (Default). Produces a CPU-readable buffer.\n")
	TEXT("0: Decode to NV12 and convert to BGRA in a UE shader. Saves a small amount of memory bandwidth\n")
	TEXT("   at the cost of an extra GPU pass."),
	ECVF_ReadOnly);

/* FAVPlayerItemLegibleOutputPushDelegate
 *****************************************************************************/

@interface FAVPlayerItemLegibleOutputPushDelegate : NSObject<AVPlayerItemLegibleOutputPushDelegate>
{
	FAvfMediaTracks* Tracks;
}
- (id)initWithMediaTracks:(FAvfMediaTracks*)Tracks;
- (void)legibleOutput:(AVPlayerItemLegibleOutput *)output didOutputAttributedStrings:(NSArray<NSAttributedString *> *)strings nativeSampleBuffers:(NSArray *)nativeSamples forItemTime:(CMTime)itemTime;
@end

@implementation FAVPlayerItemLegibleOutputPushDelegate

- (id)initWithMediaTracks:(FAvfMediaTracks*)InTracks
{
	id Self = [super init];
	if (Self)
	{
		Tracks = InTracks;
	}
	return Self;
}

- (void)legibleOutput:(AVPlayerItemLegibleOutput *)Output didOutputAttributedStrings:(NSArray<NSAttributedString *> *)Strings nativeSampleBuffers:(NSArray *)NativeSamples forItemTime:(CMTime)ItemTime
{
	Tracks->ProcessCaptions(Output, Strings, NativeSamples, ItemTime);
}

@end

#if AUDIO_PLAYBACK_VIA_ENGINE
/*
 * Audio Tap Handling callbacks see:
 * https://rymc.io/2014/01/08/recording-live-audio-streams-on-ios/
 * https://chritto.wordpress.com/2013/01/07/processing-avplayers-audio-with-mtaudioprocessingtap/
 *
 **/

struct AudioTrackTapContextData
{
	AudioStreamBasicDescription ProcessingFormat;
	AudioStreamBasicDescription DestinationFormat;
	FMediaSamples&				SampleQueue;
	FAvfMediaAudioSamplePool*	AudioSamplePool;
	bool						bActive;			// Init and shutdown flag
	volatile bool&				bMuted;				// Muted usually with -ve playback rate
	
	AudioTrackTapContextData(FMediaSamples& InSampleQueue, FAvfMediaAudioSamplePool* InAudioSamplePool, AudioStreamBasicDescription const & InDestinationFormat, volatile bool& bInBindMuted)
	: SampleQueue(InSampleQueue)
	, AudioSamplePool(InAudioSamplePool)
	, bActive(false)
	, bMuted(bInBindMuted)
	{
		FMemory::Memcpy(&DestinationFormat, &InDestinationFormat, sizeof(AudioStreamBasicDescription));
	}
};

static void AudioTrackTapInit(MTAudioProcessingTapRef __nonnull TapRef, void * __nullable Userdata, void * __nullable * __nonnull TapStorageOut)
{
	// Just pass this through
	*TapStorageOut = Userdata;
}

static void AudioTrackTapPrepare(MTAudioProcessingTapRef __nonnull TapRef, CMItemCount MaxFrames, const AudioStreamBasicDescription * __nonnull ProcessingFormat)
{
	AudioTrackTapContextData* ContextData = (AudioTrackTapContextData*)MTAudioProcessingTapGetStorage(TapRef);
	if(ContextData)
	{
		FMemory::Memcpy(&ContextData->ProcessingFormat,ProcessingFormat,sizeof(AudioStreamBasicDescription));
		ContextData->bActive = true;
	}
}

static void AudioTrackTapProcess(MTAudioProcessingTapRef __nonnull TapRef,
								 CMItemCount NumberFrames,
								 MTAudioProcessingTapFlags Flags,
								 AudioBufferList * __nonnull BufferListInOut,
								 CMItemCount * __nonnull NumberFramesOut,
								 MTAudioProcessingTapFlags * __nonnull FlagsOut)
{
	CMTimeRange TimeRange;
	OSStatus Status = MTAudioProcessingTapGetSourceAudio(TapRef, NumberFrames, BufferListInOut, FlagsOut, &TimeRange, NumberFramesOut);
	
	// For this use case this flag should not be set if it is then we need to do something about it - force end the tap
	check((Flags & kMTAudioProcessingTapFlag_EndOfStream) == 0);
	
	if(Status == noErr)
	{
		AudioTrackTapContextData* Ctx = (AudioTrackTapContextData*) MTAudioProcessingTapGetStorage(TapRef);
		
		// If we haven't got this then something has gone wrong
		// Could call through to AvfMediaTracks to do this processing but that would require exposing the function call definitation
		// in the public interface to AvfMediaTracks which seems wrong - plus we save the extra function call in time critical code!
		check(Ctx);
		
		if(Ctx->bActive && !Ctx->bMuted)
		{
			// Compute required buffer size
			uint32 BufferSize = (NumberFrames * ((Ctx->DestinationFormat.mBitsPerChannel / 8))) * Ctx->DestinationFormat.mChannelsPerFrame;
			
			//setup reasonable defaults as time range can be invalid - especially at the start of the audio track
			FTimespan StartTime(0);
			FTimespan Duration(((int64)NumberFrames * ETimespan::TicksPerSecond) / (int64)Ctx->DestinationFormat.mSampleRate);
			
			// If valid set time stamps give by the system
			if((TimeRange.start.flags & kCMTimeFlags_Valid) == kCMTimeFlags_Valid)
			{
				StartTime = (TimeRange.start.value * ETimespan::TicksPerSecond) / TimeRange.start.timescale;
			}

			// On pause the duration from system can be different from computed
			if((TimeRange.duration.flags & kCMTimeFlags_Valid) == kCMTimeFlags_Valid)
			{
				Duration = (TimeRange.duration.value * ETimespan::TicksPerSecond) / TimeRange.duration.timescale;
			}
			
			// Don't add zero duration sample buffers to to the sink
			if(Duration.GetTicks() > 0)
			{
				// Get a media audio sample buffer from the pool
				const TSharedRef<FAvfMediaAudioSample, ESPMode::ThreadSafe> AudioSample = Ctx->AudioSamplePool->AcquireShared();
				if (AudioSample->Initialize(BufferSize,
											NumberFrames,
											Ctx->DestinationFormat.mChannelsPerFrame,
											Ctx->DestinationFormat.mSampleRate,
											StartTime,
											Duration))
				{
					if((Ctx->ProcessingFormat.mFormatFlags & kAudioFormatFlagIsFloat) != 0)
					{
						float* DestBuffer = (float*)AudioSample->GetMutableBuffer();
						const uint32 BufferCount = BufferListInOut->mNumberBuffers;
						
						// We need to have the same amount of buffers as the channel count
						check(BufferCount == Ctx->DestinationFormat.mChannelsPerFrame);
						
						// Interleave the seperate channel buffers into one buffer
						for(uint32 b = 0; b < BufferCount;++b)
						{
							AudioBuffer& Buffer = BufferListInOut->mBuffers[b];
							
							// We don't handle source processing interleaved formats - if this number equals mChannelPerFrame then we could just blit the data across in one go
							check(Buffer.mNumberChannels == 1);
							
							// Make sure each channel buffer has the right about of data for the number of frames and processing format
							check(Buffer.mDataByteSize == NumberFrames * (Ctx->DestinationFormat.mBitsPerChannel / 8));
							
							float* SrcBuffer = (float*)Buffer.mData;
							
							// Perform interleave copy
							for(uint32 f = 0;f < NumberFrames;++f)
							{
								uint32 idx = b + (f * BufferCount);
								DestBuffer[idx] = SrcBuffer[f];
							}
							
							// Done with this source buffer clear it - otherwise AVPlayer will also play this audio -  we could seet a volume of 0 on the AudioMixInputParameters
							// But that could be dangerous as OS may optimise out somethings at runtime for 0 volume tracks in the future
							FMemory::Memset(Buffer.mData, 0, Buffer.mDataByteSize);
						}
					}
					else
					{
						// Processing format should always be float however...
						// If we encounter this case (kAudioFormatFlagIsSignedInteger) we need to use sint16 audio sample type on FAvfMediaAudioSample
						// i.e. make FAvfMediaAudioSample sample type settable, the engine should convert to float internally as that is it's preferred format now
						check(false);
					}
					
					Ctx->SampleQueue.AddAudio(AudioSample);
				}
			}
		}
		else
		{
			// On mute or inactive make sure no audio 'leaks' through to the OS mixer - we can't rely on the outer AVPlayer when muted with this tap attached to be fast or clean about dealing with this
			const uint32 BufferCount = BufferListInOut->mNumberBuffers;
			for(uint32 b = 0;b < BufferCount;++b)
			{
				AudioBuffer& Buffer = BufferListInOut->mBuffers[b];
				FMemory::Memset(Buffer.mData, 0, Buffer.mDataByteSize);
			}
		}
	}
}

static void AudioTrackTapUnPrepare(MTAudioProcessingTapRef __nonnull TapRef)
{
	// NOP
}

static void AudioTrackTapFinalize(MTAudioProcessingTapRef __nonnull TapRef)
{
	AudioTrackTapContextData* Context = (AudioTrackTapContextData*)MTAudioProcessingTapGetStorage(TapRef);
	if(Context)
	{
		delete Context;
		Context = nullptr;
	}
}

static void AudioTrackTapShutdownCurrentAudioTrackProcessing(AVPlayerItem* PlayerItem)
{
	SCOPED_AUTORELEASE_POOL;

	check(PlayerItem);
	
	if(PlayerItem.audioMix != nil)
	{
		MTAudioProcessingTapRef TapNodeRef = ((AVMutableAudioMixInputParameters*)PlayerItem.audioMix.inputParameters[0]).audioTapProcessor;
		
		PlayerItem.audioMix = nil;
		
		if(TapNodeRef)
		{
			AudioTrackTapContextData* Ctx = (AudioTrackTapContextData*) MTAudioProcessingTapGetStorage(TapNodeRef);
			check(Ctx);
			
			Ctx->bActive = false;
		}
	}
}

static void AudioTrackTapInitializeForAudioTrack(FMediaSamples& InSampleQueue, FAvfMediaAudioSamplePool* InAudioSamplePool, AudioStreamBasicDescription const & InDestinationFormat, AVPlayerItem* PlayerItem, AVAssetTrack* AssetTrack, volatile bool& bInBindMuted)
{
	SCOPED_AUTORELEASE_POOL;

	check(InAudioSamplePool);
	check(PlayerItem);
	
	AudioTrackTapShutdownCurrentAudioTrackProcessing(PlayerItem);
	
	if(AssetTrack != nil)
	{
		MTAudioProcessingTapCallbacks Callbacks;
		
		Callbacks.version = kMTAudioProcessingTapCallbacksVersion_0;
		Callbacks.clientInfo = new AudioTrackTapContextData(InSampleQueue, InAudioSamplePool, InDestinationFormat, bInBindMuted);
		Callbacks.init = AudioTrackTapInit;
		Callbacks.prepare = AudioTrackTapPrepare;
		Callbacks.process = AudioTrackTapProcess;
		Callbacks.unprepare = AudioTrackTapUnPrepare;
		Callbacks.finalize = AudioTrackTapFinalize;
		
		MTAudioProcessingTapRef TapNodeRef = nullptr;
		
		OSStatus ErrorState = MTAudioProcessingTapCreate(kCFAllocatorDefault, &Callbacks, kMTAudioProcessingTapCreationFlag_PreEffects, &TapNodeRef);
		
		if (ErrorState == noErr && TapNodeRef != nullptr)
		{
			AVMutableAudioMixInputParameters* InputParams = [AVMutableAudioMixInputParameters audioMixInputParametersWithTrack:AssetTrack];
			
			InputParams.audioTapProcessor = TapNodeRef;
			InputParams.trackID = AssetTrack.trackID;
			
			AVMutableAudioMix* AudioMix = [AVMutableAudioMix audioMix];
			AudioMix.inputParameters = @[InputParams];
			
			PlayerItem.audioMix = AudioMix;

			CFRelease(TapNodeRef);
		}
	}
}

#endif //AUDIO_PLAYBACK_VIA_ENGINE


/* FAvfMediaTracks structors
 *****************************************************************************/

FAvfMediaTracks::FAvfMediaTracks(FMediaSamples& InSamples)
	: AudioSamplePool(new FAvfMediaAudioSamplePool)
	, PlayerItem(nullptr)
	, Samples(InSamples)
	, SelectedAudioTrack(INDEX_NONE)
	, SelectedCaptionTrack(INDEX_NONE)
	, SelectedVideoTrack(INDEX_NONE)
{
	VideoSampler = MakeShared<FAvfMediaVideoSampler, ESPMode::ThreadSafe>(Samples);
}


FAvfMediaTracks::~FAvfMediaTracks()
{
	Reset();

	delete AudioSamplePool;
	AudioSamplePool = nullptr;
}


/* FAvfMediaTracks interface
 *****************************************************************************/

void FAvfMediaTracks::AppendStats(FString &OutStats) const
{
	FScopeLock Lock(&CriticalSection);

	// audio tracks
	OutStats += TEXT("Audio Tracks\n");

	if (AudioTracks.Num() == 0)
	{
		OutStats += TEXT("    none\n");
	}
	else
	{
		for (uint32 TrackIndex = 0; TrackIndex < AudioTracks.Num(); TrackIndex++)
		{
			const FTrack& Track = AudioTracks[TrackIndex];

			OutStats += FString::Printf(TEXT("    %s\n"), *GetTrackDisplayName(EMediaTrackType::Audio, TrackIndex).ToString());
			OutStats += FString::Printf(TEXT("        Not implemented yet"));
		}
	}

	// video tracks
	OutStats += TEXT("Video Tracks\n");

	if (VideoTracks.Num() == 0)
	{
		OutStats += TEXT("    none\n");
	}
	else
	{
		for (uint32 TrackIndex = 0; TrackIndex < VideoTracks.Num(); TrackIndex++)
		{
			const FTrack& Track = VideoTracks[TrackIndex];

			OutStats += FString::Printf(TEXT("    %s\n"), *GetTrackDisplayName(EMediaTrackType::Video, TrackIndex).ToString());
			OutStats += FString::Printf(TEXT("        BitRate: %i\n"), int([Track.AssetTrack estimatedDataRate]));
		}
	}
}


void FAvfMediaTracks::Initialize(AVPlayerItem* InPlayerItem, FString& OutInfo)
{
	Reset();

	FScopeLock Lock(&CriticalSection);

	PlayerItem = [InPlayerItem retain];

	// initialize tracks
	NSArray* PlayerTracks = PlayerItem.tracks;
	NSError* Error = nil;

	int32 StreamIndex = 0;

	for (AVPlayerItemTrack* PlayerTrack in PlayerItem.tracks)
	{
		// create track
		FTrack* Track = nullptr;
		AVAssetTrack* AssetTrack = PlayerTrack.assetTrack;
		NSString* MediaType = AssetTrack.mediaType;

		OutInfo += FString::Printf(TEXT("Stream %i\n"), StreamIndex);
		OutInfo += FString::Printf(TEXT("    Type: %s\n"), *AvfMedia::MediaTypeToString(MediaType));

		if ([MediaType isEqualToString:AVMediaTypeAudio])
		{
			int32 TrackIndex = AudioTracks.AddDefaulted();
			Track = &AudioTracks[TrackIndex];
			
			Track->Name = FString::Printf(TEXT("Audio Track %i"), TrackIndex);
			Track->Output = [PlayerTrack retain];
			Track->Loaded = true;

			CMFormatDescriptionRef DescRef = (CMFormatDescriptionRef)[AssetTrack.formatDescriptions objectAtIndex:0];
			const AudioStreamBasicDescription* Desc = CMAudioFormatDescriptionGetStreamBasicDescription(DescRef);

			if (Desc)
			{
				OutInfo += FString::Printf(TEXT("    Channels: %u\n"), Desc->mChannelsPerFrame);
				OutInfo += FString::Printf(TEXT("    Sample Rate: %g Hz\n"), Desc->mSampleRate);
				if (Desc->mBitsPerChannel > 0)
				{
					OutInfo += FString::Printf(TEXT("    Bits Per Channel: %u\n"), Desc->mBitsPerChannel);
				}
				else
				{
					OutInfo += FString::Printf(TEXT("    Bits Per Channel: n/a\n"));
				}
			}
			else
			{
				OutInfo += TEXT("    failed to get audio track information\n");
			}
		}
		else if (([MediaType isEqualToString:AVMediaTypeClosedCaption]) || ([MediaType isEqualToString:AVMediaTypeSubtitle]) || ([MediaType isEqualToString:AVMediaTypeText]))
		{
			FAVPlayerItemLegibleOutputPushDelegate* Delegate = [[FAVPlayerItemLegibleOutputPushDelegate alloc] initWithMediaTracks:this];
			AVPlayerItemLegibleOutput* Output = [AVPlayerItemLegibleOutput new];
			check(Output);
			
			// We don't want AVPlayer to render the frame, just decode it for us
			Output.suppressesPlayerRendering = YES;
			
			[Output setDelegate:Delegate queue:dispatch_get_global_queue(QOS_CLASS_DEFAULT, 0)];

			int32 TrackIndex = CaptionTracks.AddDefaulted();
			Track = &CaptionTracks[TrackIndex];

			Track->Name = FString::Printf(TEXT("Caption Track %i"), TrackIndex);
			Track->Output = Output;
			Track->Loaded = true;
		}
		else if ([MediaType isEqualToString:AVMediaTypeTimecode])
		{
			// not implemented yet - not sure they should be as these are SMTPE editing timecodes for iMovie/Final Cut/etc. not playback timecodes. They only make sense in editable Quicktime Movies (.mov).
			OutInfo += FString::Printf(TEXT("    Type: Timecode (UNSUPPORTED)\n"));
		}
		else if ([MediaType isEqualToString:AVMediaTypeVideo])
		{
			NSMutableDictionary* OutputSettings = [NSMutableDictionary dictionary];
			CMFormatDescriptionRef DescRef = (CMFormatDescriptionRef)[AssetTrack.formatDescriptions objectAtIndex:0];
			CMVideoCodecType CodecType = CMFormatDescriptionGetMediaSubType(DescRef);
			
			// Select decode pixel format - BGRA32 is the fallback - Any more pixel formats added here need to be handled correctly in FAvfMediaVideoSampler
			int32 DecodePixelBufferFormat = kCVPixelFormatType_32BGRA;
	
			if(!AVFMediaForceDecodeBGRA)
			{
				if(kCMVideoCodecType_H264 == CodecType)
				{
					DecodePixelBufferFormat = kCVPixelFormatType_420YpCbCr8BiPlanarVideoRange;
					
					CFDictionaryRef FormatExtensions = CMFormatDescriptionGetExtensions(DescRef);
					if(FormatExtensions)
					{
						CFBooleanRef bFullRange = (CFBooleanRef)CFDictionaryGetValue(FormatExtensions, kCMFormatDescriptionExtension_FullRangeVideo);
						if(bFullRange && (bool)CFBooleanGetValue(bFullRange))
						{
							DecodePixelBufferFormat = kCVPixelFormatType_420YpCbCr8BiPlanarFullRange;
						}
					}
				}
			}
			
			[OutputSettings setObject : [NSNumber numberWithInt :  DecodePixelBufferFormat] forKey : (NSString*)kCVPixelBufferPixelFormatTypeKey];
			
#if WITH_ENGINE
			
			[OutputSettings setObject:[NSNumber numberWithBool:YES] forKey:(NSString*)kCVPixelBufferMetalCompatibilityKey];
			
#endif //WITH_ENGINE

			// Use unaligned rows
			[OutputSettings setObject:[NSNumber numberWithInteger:1] forKey:(NSString*)kCVPixelBufferBytesPerRowAlignmentKey];
		
			// Then create the video output object from which we will grab frames as CVPixelBuffer's
			AVPlayerItemVideoOutput* Output = [[AVPlayerItemVideoOutput alloc] initWithPixelBufferAttributes:OutputSettings];
			check(Output);
			// We don't want AVPlayer to render the frame, just decode it for us
			Output.suppressesPlayerRendering = YES;

			int32 TrackIndex = VideoTracks.AddDefaulted();
			Track = &VideoTracks[TrackIndex];

			Track->Name = FString::Printf(TEXT("Video Track %i"), TrackIndex);
			Track->Output = Output;
			Track->Loaded = true;
			Track->bFullRangeVideo = DecodePixelBufferFormat != kCVPixelFormatType_420YpCbCr8BiPlanarVideoRange;
			
			// NominalFrameRate can be zero (e.g HLS streams) - try again using min frame duration, otherwise it's unknown - possibly variable - use a default
			Track->FrameRate = AssetTrack.nominalFrameRate;
			if(Track->FrameRate <= 0.f)
			{
				CMTime frameDuration = AssetTrack.minFrameDuration;
				if((frameDuration.flags & kCMTimeFlags_Valid) != 0)
				{
					Track->FrameRate = (float)frameDuration.timescale / (float)frameDuration.value;
				}
				else
				{
					Track->FrameRate = 24.f;
				}
			}
			
			CGSize NaturalFrameSize = AssetTrack.naturalSize;
			Track->FrameSize = FIntPoint((int32)NaturalFrameSize.width, (int32)NaturalFrameSize.height);

			OutInfo += FString::Printf(TEXT("    Codec: %s\n"), *AvfMedia::CodecTypeToString(CodecType));
			OutInfo += FString::Printf(TEXT("    Dimensions: %i x %i\n"), (int32)NaturalFrameSize.width, (int32)NaturalFrameSize.height);
			OutInfo += FString::Printf(TEXT("    Frame Rate: %g fps\n"), AssetTrack.nominalFrameRate);
			OutInfo += FString::Printf(TEXT("    BitRate: %i\n"), (int32)AssetTrack.estimatedDataRate);
		}

		OutInfo += TEXT("\n");

		PlayerTrack.enabled = NO;

		if (Track != nullptr)
		{
			Track->AssetTrack = AssetTrack;
			Track->DisplayName = FText::FromString(Track->Name);
			Track->StreamIndex = StreamIndex;
		}

		++StreamIndex;
	}
}

void FAvfMediaTracks::ProcessCaptions(AVPlayerItemLegibleOutput* Output, NSArray<NSAttributedString*>* Strings, NSArray* NativeSamples, CMTime ItemTime)
{
	if (SelectedCaptionTrack != INDEX_NONE)
	{
		return;
	}

	FScopeLock Lock(&CriticalSection);
		
	NSDictionary* DocumentAttributes = @{NSDocumentTypeDocumentAttribute:NSPlainTextDocumentType};
	FTimespan DisplayTime = FTimespan::FromSeconds(CMTimeGetSeconds(ItemTime));
	
	FString OutputString;
	bool bFirst = true;

	for (NSAttributedString* String in Strings)
	{
		if (!String)
		{
			continue;
		}

		// strip attributes from the string (we don't care for them)
		NSRange Range = NSMakeRange(0, String.length);
		NSData* Data = [String dataFromRange:Range documentAttributes:DocumentAttributes error:NULL];
		NSString* Result = [[NSString alloc] initWithData:Data encoding:NSUTF8StringEncoding];
				
		// append the string
		if (!bFirst)
		{
			OutputString += TEXT("\n");
		}

		bFirst = false;
		OutputString += FString(Result);
	}

	if (OutputString.IsEmpty())
	{
		return;
	}

	// create & add sample to queue
	auto OverlaySample = MakeShared<FAvfMediaOverlaySample, ESPMode::ThreadSafe>();

	if (OverlaySample->Initialize(OutputString, DisplayTime))
	{
		Samples.AddCaption(OverlaySample);
	}
}


void FAvfMediaTracks::ProcessVideo()
{
	struct FVideoSamplerTickParams
	{
		TWeakPtr<FAvfMediaVideoSampler, ESPMode::ThreadSafe> VideoSamplerPtr;
	}
	Params = { VideoSampler };

	ENQUEUE_RENDER_COMMAND(AvfMediaVideoSamplerTick)(
		[Params](FRHICommandListImmediate& RHICmdList)
		{
			auto PinnedVideoSampler = Params.VideoSamplerPtr.Pin();

			if (PinnedVideoSampler.IsValid())
			{
				PinnedVideoSampler->Tick();
			}
		});
}


void FAvfMediaTracks::Reset()
{
	FScopeLock Lock(&CriticalSection);

	// reset video sampler
	struct FResetOutputParams
	{
		TWeakPtr<FAvfMediaVideoSampler, ESPMode::ThreadSafe> VideoSamplerPtr;
	}
	Params = { VideoSampler };

	ENQUEUE_RENDER_COMMAND(AvfMediaVideoSamplerResetOutput)(
		[Params](FRHICommandListImmediate& RHICmdList)
	    {
			auto PinnedVideoSampler = Params.VideoSamplerPtr.Pin();

			if (PinnedVideoSampler.IsValid())
			{
				PinnedVideoSampler->SetOutput(nil, 0.0f);
				PinnedVideoSampler->Reset();
			}
		});
	
	// reset tracks
	SelectedAudioTrack = INDEX_NONE;
	SelectedCaptionTrack = INDEX_NONE;
	SelectedVideoTrack = INDEX_NONE;

	for (FTrack& Track : AudioTracks)
	{
		[Track.Output release];
	}

	for (FTrack& Track : CaptionTracks)
	{
		AVPlayerItemLegibleOutput* Output = (AVPlayerItemLegibleOutput*)Track.Output;

		[Output.delegate release];
		[Track.Output release];
	}

	for (FTrack& Track : VideoTracks)
	{
		[Track.Output release];
	}

	AudioTracks.Empty();
	CaptionTracks.Empty();
	VideoTracks.Empty();
	
	if(PlayerItem != nil)
	{
#if AUDIO_PLAYBACK_VIA_ENGINE
		AudioTrackTapShutdownCurrentAudioTrackProcessing(PlayerItem);
#endif
		[PlayerItem release];
	}
	
	PlayerItem = nil;
	
#if AUDIO_PLAYBACK_VIA_ENGINE
	bMuted = false;
#endif
}

#if AUDIO_PLAYBACK_VIA_ENGINE
void FAvfMediaTracks::ApplyMuteState(bool bMute)
{
	bMuted = bMute;
}
#endif

/* IMediaTracks interface
 *****************************************************************************/

bool FAvfMediaTracks::GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	if ((FormatIndex != 0) || !AudioTracks.IsValidIndex(TrackIndex))
	{
		return false;
	}

	const FTrack& Track = AudioTracks[TrackIndex];
	checkf(Track.AssetTrack.formatDescriptions.count == 1, TEXT("Can't handle non-uniform audio streams!"));

	CMFormatDescriptionRef DescRef = (CMFormatDescriptionRef)[AudioTracks[TrackIndex].AssetTrack.formatDescriptions objectAtIndex : 0];
	AudioStreamBasicDescription const* Desc = CMAudioFormatDescriptionGetStreamBasicDescription(DescRef);

	OutFormat.BitsPerSample = 32;
	OutFormat.NumChannels = (Desc != nullptr) ? Desc->mChannelsPerFrame : 0;
	OutFormat.SampleRate = (Desc != nullptr) ? Desc->mSampleRate : 0;
	OutFormat.TypeName = TEXT("PCM"); // @todo trepka: fix me (should be input type, not output type)

	return true;
}


int32 FAvfMediaTracks::GetNumTracks(EMediaTrackType TrackType) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return AudioTracks.Num();

	case EMediaTrackType::Caption:
		return CaptionTracks.Num();

	case EMediaTrackType::Video:
		return VideoTracks.Num();

	default:
		return 0;
	}
}


int32 FAvfMediaTracks::GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const
{
	return ((TrackIndex >= 0) && (TrackIndex < GetNumTracks(TrackType))) ? 1 : 0;
}


int32 FAvfMediaTracks::GetSelectedTrack(EMediaTrackType TrackType) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return SelectedAudioTrack;

	case EMediaTrackType::Caption:
		return SelectedCaptionTrack;

	case EMediaTrackType::Video:
		return SelectedVideoTrack;

	default:
		return INDEX_NONE;
	}
}


FText FAvfMediaTracks::GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].DisplayName;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return CaptionTracks[TrackIndex].DisplayName;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].DisplayName;
		}
		break;

	default:
		break;
	}

	return FText::GetEmpty();
}


int32 FAvfMediaTracks::GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const
{
	return (GetSelectedTrack(TrackType) != INDEX_NONE) ? 0 : INDEX_NONE;
}


FString FAvfMediaTracks::GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const
{
	if (TrackType == EMediaTrackType::Audio)
	{
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return FString(AudioTracks[TrackIndex].AssetTrack.languageCode);
		}
	}
	else if (TrackType == EMediaTrackType::Caption)
	{
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return FString(CaptionTracks[TrackIndex].AssetTrack.languageCode);
		}
	}
	else if (TrackType == EMediaTrackType::Video)
	{
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return FString(VideoTracks[TrackIndex].AssetTrack.languageCode);
		}
	}

	return FString();
}


FString FAvfMediaTracks::GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].Name;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return CaptionTracks[TrackIndex].Name;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].Name;
		}
		break;

	default:
		break;
	}

	return FString();
}

bool FAvfMediaTracks::GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	if ((FormatIndex != 0) || !VideoTracks.IsValidIndex(TrackIndex))
	{
		return false;
	}

	const FTrack& Track = VideoTracks[TrackIndex];
	
	OutFormat.Dim = Track.FrameSize;
	OutFormat.FrameRate = Track.FrameRate;
	OutFormat.FrameRates = TRange<float>(OutFormat.FrameRate);
	OutFormat.TypeName = TEXT("BGRA"); // @todo trepka: fix me (should be input format, not output format)

	return true;
}

bool FAvfMediaTracks::SelectTrack(EMediaTrackType TrackType, int32 TrackIndex)
{
	FScopeLock Lock(&CriticalSection);

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (TrackIndex != SelectedAudioTrack)
		{
			UE_LOGF(LogAvfMedia, Verbose, "Selecting audio track %i instead of %i (%i tracks).", TrackIndex, SelectedAudioTrack, AudioTracks.Num());

			// disable current track
			if (SelectedAudioTrack != INDEX_NONE)
			{
				UE_LOGF(LogAvfMedia, VeryVerbose, "Disabling audio track %i", SelectedAudioTrack);
				
				AVPlayerItemTrack* PlayerTrack = (AVPlayerItemTrack*)AudioTracks[SelectedAudioTrack].Output;
				check(PlayerTrack);
				PlayerTrack.enabled = NO;
				
#if AUDIO_PLAYBACK_VIA_ENGINE
				AudioTrackTapShutdownCurrentAudioTrackProcessing(PlayerItem);
#endif

				SelectedAudioTrack = INDEX_NONE;
			}

			// enable new track
			if (TrackIndex != INDEX_NONE)
			{
				if (!AudioTracks.IsValidIndex(TrackIndex))
				{
					return false;
				}

				UE_LOGF(LogAvfMedia, VeryVerbose, "Enabling audio track %i", TrackIndex);
			}

			SelectedAudioTrack = TrackIndex;

			// update output
			if (SelectedAudioTrack != INDEX_NONE)
			{
				const FTrack& SelectedTrack = AudioTracks[SelectedAudioTrack];
				PlayerItem.tracks[SelectedTrack.StreamIndex].enabled = YES;

#if AUDIO_PLAYBACK_VIA_ENGINE
				CMFormatDescriptionRef DescRef = (CMFormatDescriptionRef)[SelectedTrack.AssetTrack.formatDescriptions objectAtIndex : 0];
				AudioStreamBasicDescription const* ASBD = CMAudioFormatDescriptionGetStreamBasicDescription(DescRef);
				
				TargetDesc.mSampleRate = ASBD->mSampleRate;
				TargetDesc.mFormatID = kAudioFormatLinearPCM;
				TargetDesc.mFormatFlags = kAudioFormatFlagIsFloat | kAudioFormatFlagsNativeEndian | kAudioFormatFlagIsPacked;
				TargetDesc.mFramesPerPacket = 1;
				TargetDesc.mBytesPerFrame = ASBD->mChannelsPerFrame * sizeof(float);
				TargetDesc.mBytesPerPacket = TargetDesc.mBytesPerFrame * TargetDesc.mFramesPerPacket;
				TargetDesc.mChannelsPerFrame = ASBD->mChannelsPerFrame;
				TargetDesc.mBitsPerChannel = 32;
				TargetDesc.mReserved = 0;
				
				AudioTrackTapInitializeForAudioTrack(Samples, AudioSamplePool, TargetDesc, PlayerItem, SelectedTrack.AssetTrack, bMuted);
#endif
				
				AVPlayerItemTrack* PlayerTrack = (AVPlayerItemTrack*)SelectedTrack.Output;
				check(PlayerTrack);

				PlayerTrack.enabled = YES;
			}
		}
		break;

	case EMediaTrackType::Caption:
		if (TrackIndex != SelectedCaptionTrack)
		{
			UE_LOGF(LogAvfMedia, Verbose, "Selecting caption track %i instead of %i (%i tracks).", TrackIndex, SelectedCaptionTrack, CaptionTracks.Num());

			// disable current track
			if (SelectedCaptionTrack != INDEX_NONE)
			{
				UE_LOGF(LogAvfMedia, VeryVerbose, "Disabling caption track %i", SelectedCaptionTrack);

				const FTrack& Track = CaptionTracks[SelectedCaptionTrack];

				[PlayerItem removeOutput : (AVPlayerItemOutput*)Track.Output];
				PlayerItem.tracks[Track.StreamIndex].enabled = NO;

				SelectedCaptionTrack = INDEX_NONE;
			}

			// enable new track
			if (TrackIndex != INDEX_NONE)
			{
				if (!CaptionTracks.IsValidIndex(TrackIndex))
				{
					return false;
				}

				UE_LOGF(LogAvfMedia, VeryVerbose, "Enabling caption track %i", TrackIndex);

				const FTrack& SelectedTrack = CaptionTracks[TrackIndex];
				PlayerItem.tracks[SelectedTrack.StreamIndex].enabled = YES;
			}

			SelectedCaptionTrack = TrackIndex;

			// update output
			if (SelectedCaptionTrack != INDEX_NONE)
			{
				[PlayerItem addOutput:(AVPlayerItemOutput*)CaptionTracks[SelectedCaptionTrack].Output];
			}
		}
		break;

	case EMediaTrackType::Video:
		if (TrackIndex != SelectedVideoTrack)
		{
			UE_LOGF(LogAvfMedia, Verbose, "Selecting video track %i instead of %i (%i tracks)", TrackIndex, SelectedVideoTrack, VideoTracks.Num());

			// disable current track
			if (SelectedVideoTrack != INDEX_NONE)
			{
				UE_LOGF(LogAvfMedia, VeryVerbose, "Disabling video track %i", SelectedVideoTrack);

				const FTrack& Track = VideoTracks[SelectedVideoTrack];

				[PlayerItem removeOutput : (AVPlayerItemOutput*)Track.Output];
				PlayerItem.tracks[Track.StreamIndex].enabled = NO;

				SelectedVideoTrack = INDEX_NONE;
			}

			// enable new track
			if (TrackIndex != INDEX_NONE)
			{
				if (!VideoTracks.IsValidIndex(TrackIndex))
				{
					return false;
				}

				UE_LOGF(LogAvfMedia, VeryVerbose, "Enabling video track %i", TrackIndex);

				const FTrack& SelectedTrack = VideoTracks[TrackIndex];
				PlayerItem.tracks[SelectedTrack.StreamIndex].enabled = YES;
			}

			SelectedVideoTrack = TrackIndex;

			// update output
			if (SelectedVideoTrack != INDEX_NONE)
			{
				[PlayerItem addOutput : (AVPlayerItemOutput*)VideoTracks[SelectedVideoTrack].Output];

				struct FSetOutputParams
				{
					AVPlayerItemVideoOutput* Output;
					TWeakPtr<FAvfMediaVideoSampler, ESPMode::ThreadSafe> VideoSamplerPtr;
					float FrameRate;
					bool bFullRange;
				}
				Params = {
					(AVPlayerItemVideoOutput*)VideoTracks[SelectedVideoTrack].Output,
					VideoSampler,
					VideoTracks[TrackIndex].FrameRate,
					VideoTracks[TrackIndex].bFullRangeVideo
				};

				ENQUEUE_RENDER_COMMAND(AvfMediaVideoSamplerSetOutput)(
					[Params](FRHICommandListImmediate& RHICmdList)
					{
						auto PinnedVideoSampler = Params.VideoSamplerPtr.Pin();

						if (PinnedVideoSampler.IsValid())
						{
							PinnedVideoSampler->SetOutput(Params.Output, Params.FrameRate, Params.bFullRange);
						}
					});
			}
		}
		break;

	default:
		return false;
	}

	return true;
}


bool FAvfMediaTracks::SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex)
{
	if (FormatIndex != 0)
	{
		return false;
	}

	FScopeLock Lock(&CriticalSection);

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return AudioTracks.IsValidIndex(TrackIndex);

	case EMediaTrackType::Caption:
		return CaptionTracks.IsValidIndex(TrackIndex);

	case EMediaTrackType::Video:
		return VideoTracks.IsValidIndex(TrackIndex);

	default:
		return false;
	}
}

NS_ASSUME_NONNULL_END
