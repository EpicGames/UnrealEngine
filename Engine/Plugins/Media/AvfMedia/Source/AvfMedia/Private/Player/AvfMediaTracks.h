// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "IMediaSamples.h"
#include "IMediaTracks.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import <AVFoundation/AVFoundation.h>

#define AUDIO_PLAYBACK_VIA_ENGINE (PLATFORM_MAC)

class FAvfMediaAudioSamplePool;
class FAvfMediaVideoSamplePool;
class FAvfMediaVideoSampler;
class FMediaSamples;


/**
 * 
 */
class FAvfMediaTracks
	: public IMediaTracks
{
	enum ESyncStatus
	{
		Default,    // Starting state
		Behind,     // Frame is behind playback cursor.
		Ready,      // Frame is within tolerance of playback cursor.
		Ahead,      // Frame is ahead of playback cursor.
	};

	struct FTrack
	{
		FTrack()
		: Loaded(false)
		, FrameSize(0, 0)
		, FrameRate(0.f)
		, bFullRangeVideo(false)
		{ }
		
		AVAssetTrack* AssetTrack;
		FText DisplayName;
		bool Loaded;
		FString Name;
		NSObject* Output;
		int32 StreamIndex;

		//Cached Video Track Data
		FIntPoint FrameSize;
		float FrameRate;
		bool bFullRangeVideo;
	};

public:

	/** Default constructor. */
	FAvfMediaTracks(FMediaSamples& InSamples);

	/** Virtual destructor. */
	virtual ~FAvfMediaTracks();

public:

	/**
	 * Append track statistics information to the given string.
	 *
	 * @param OutStats The string to append the statistics to.
	 */
	void AppendStats(FString &OutStats) const;

	/**
	 * Initialize the track collection.
	 *
	 * @param PlayerItem The player item containing the track information.
	 * @param OutInfo Will contain information about the available media tracks.
	 */
	void Initialize(AVPlayerItem* InPlayerItem, FString& OutInfo);

	/**
	 * Process caption frames.
	 *
	 * Called by the caption track delegate to provide the attributed strings
	 * for each timecode to the caption sink.
	 *
	 * @param Output The caption track output that generated the strings.
	 * @param Strings The attributed caption strings.
	 * @param ItemTime The display time for the strings.
	 * @see ProcessAudio, ProcessVideo
	 */
	void ProcessCaptions(AVPlayerItemLegibleOutput* Output, NSArray<NSAttributedString*>* Strings, NSArray* NativeSamples, CMTime ItemTime);

	/**
	 * Process video frames.
	 *
	 * @see ProcessAudio, ProcessCaptions
	 */
	void ProcessVideo();

	/** Reset the stream collection. */
	void Reset();
	
#if AUDIO_PLAYBACK_VIA_ENGINE
	/**
	 * Allow independant audio mute when producing audio buffers for play back through the engine
	 * Muting will stop sending audio buffers to the media audio sink
	 * e.g. Gives the option to have fast mute on reverse otherwise we can get a few bad buffers
	 */
	void ApplyMuteState(bool bMute);
#endif

public:

	//~ IMediaTracks interface

	virtual bool GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const override;
	virtual int32 GetNumTracks(EMediaTrackType TrackType) const override;
	virtual int32 GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetSelectedTrack(EMediaTrackType TrackType) const override;
	virtual FText GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual bool GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const override;
	virtual bool SelectTrack(EMediaTrackType TrackType, int32 TrackIndex) override;
	virtual bool SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex) override;

private:

	/** The available audio tracks. */
	TArray<FTrack> AudioTracks;

	/** The available caption tracks. */
	TArray<FTrack> CaptionTracks;

	/** The available video tracks. */
	TArray<FTrack> VideoTracks;

private:

	/** Audio sample object pool. */
	FAvfMediaAudioSamplePool* AudioSamplePool;

	/** Synchronizes write access to track arrays, selections & sinks. */
	mutable FCriticalSection CriticalSection;

	/** The player item containing the track information. */
	AVPlayerItem* PlayerItem;
	
#if AUDIO_PLAYBACK_VIA_ENGINE
	/** Current Mute State. */
	volatile bool bMuted;
#endif

	/** The media sample queue. */
	FMediaSamples& Samples;

	/** Index of the selected audio track. */
	int32 SelectedAudioTrack;

	/** Index of the selected caption track. */
	int32 SelectedCaptionTrack;

	/** Index of the selected video track. */
	int32 SelectedVideoTrack;

	/** Target description for audio output required by Media framework audio sinks. */
	AudioStreamBasicDescription TargetDesc;
	
	/** Object to sample video frames */
	TSharedPtr<FAvfMediaVideoSampler, ESPMode::ThreadSafe> VideoSampler;
};
