// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvfMediaVideoSampler.h"
#include "AvfMediaPrivate.h"

#include "Containers/ResourceArray.h"
#include "MediaSamples.h"
#include "Misc/ScopeLock.h"

#if WITH_ENGINE
#include "RenderingThread.h"
#include "RHI.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "MediaShaders.h"
#include "StaticBoundShaderState.h"
#include "Misc/ScopeLock.h"
#include "PipelineStateCache.h"
#include "RHIStaticStates.h"
#include "UObject/Class.h"
#include "IMetalDynamicRHI.h"
#endif

#include "AvfMediaTextureSample.h"



/**
 * Allows for direct GPU mem allocation for texture resource from a CVImageBufferRef's system memory backing store.
 */
class FAvfTexture2DResourceMem
	: public FResourceBulkDataInterface
{
public:
	FAvfTexture2DResourceMem(CVImageBufferRef InImageBuffer)
	: ImageBuffer(InImageBuffer)
	{
		check(ImageBuffer);
		CFRetain(ImageBuffer);
	}
	
	/**
	 * @return ptr to the resource memory which has been preallocated
	 */
	virtual const void* GetResourceBulkData() const override
	{
		CVPixelBufferLockBaseAddress(ImageBuffer, kCVPixelBufferLock_ReadOnly);
		return CVPixelBufferGetBaseAddress(ImageBuffer);
	}
	
	/**
	 * @return size of resource memory
	 */
	virtual uint32 GetResourceBulkDataSize() const override
	{
		int32 Pitch = CVPixelBufferGetBytesPerRow(ImageBuffer);
		int32 Height = CVPixelBufferGetHeight(ImageBuffer);
		uint32 Size = (Pitch * Height);

		return Size;
	}
	
	/**
	 * Free memory after it has been used to initialize RHI resource
	 */
	virtual void Discard() override
	{
		CVPixelBufferUnlockBaseAddress(ImageBuffer, kCVPixelBufferLock_ReadOnly);
		delete this;
	}
	
	virtual ~FAvfTexture2DResourceMem()
	{
		CFRelease(ImageBuffer);
		ImageBuffer = nullptr;
	}
	
	CVImageBufferRef ImageBuffer;
};


/* FAvfMediaVideoSampler structors
 *****************************************************************************/

FAvfMediaVideoSampler::FAvfMediaVideoSampler(FMediaSamples& InSamples)
	: Output(nil)
	, Samples(InSamples)
	, VideoSamplePool(new FAvfMediaTextureSamplePool)
	, FrameDuration(0.0f)
	, ColorTransform(nullptr)
#if WITH_ENGINE
	, MetalTextureCache(nullptr)
#endif
{ }


FAvfMediaVideoSampler::~FAvfMediaVideoSampler()
{
	[Output release];

	delete VideoSamplePool;
	VideoSamplePool = nullptr;

#if WITH_ENGINE
	if (MetalTextureCache)
	{
		CFRelease(MetalTextureCache);
		MetalTextureCache = nullptr;
	}
#endif
}


/* FAvfMediaVideoSampler interface
 *****************************************************************************/

void FAvfMediaVideoSampler::SetOutput(AVPlayerItemVideoOutput* InOutput, float InFrameRate, bool bFullRange)
{
	check(IsInRenderingThread());

	FScopeLock Lock(&CriticalSection);

	[InOutput retain];
	[Output release];
	Output = InOutput;

	FrameDuration = 1.f / InFrameRate;
	
	if(bFullRange)
	{
		ColorTransform = &MediaShaders::YuvToRgbRec709Scaled;
	}
	else
	{
		ColorTransform = &MediaShaders::YuvToRgbRec709Unscaled;
	}
}

void FAvfMediaVideoSampler::Reset()
{
	auto VideoSample = VideoSamplePool->AcquireShared();
	VideoSample->Reset();
}

void FAvfMediaVideoSampler::Tick()
{
	check(IsInRenderingThread());

	FScopeLock Lock(&CriticalSection);

	if (Output == nil)
	{
		return;
	}

	CMTime OutputItemTime = [Output itemTimeForHostTime:CACurrentMediaTime()];

	if (![Output hasNewPixelBufferForItemTime : OutputItemTime])
	{
		return;
	}

	CVPixelBufferRef Frame = [Output copyPixelBufferForItemTime:OutputItemTime itemTimeForDisplay:nullptr];

	if (!Frame)
	{
		return;
	}

	const FTimespan SampleDuration = FTimespan::FromSeconds(FrameDuration);
	const FTimespan SampleTime = FTimespan::FromSeconds(CMTimeGetSeconds(OutputItemTime));

	ProcessFrame(Frame, SampleTime, SampleDuration);
}

void FAvfMediaVideoSampler::ProcessFrame(CVPixelBufferRef Frame, FTimespan SampleTime, FTimespan SampleDuration)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	const int32 FrameHeight = CVPixelBufferGetHeight(Frame);
	const int32 FrameWidth = CVPixelBufferGetWidth(Frame);
	const int32 FrameStride = CVPixelBufferGetBytesPerRow(Frame);

	const FIntPoint Dim = FIntPoint(FrameStride / 4, FrameHeight);
	const FIntPoint OutputDim = FIntPoint(FrameWidth, FrameHeight);

	auto VideoSample = VideoSamplePool->AcquireShared();

#if WITH_ENGINE
	TRefCountPtr<FRHITexture> ShaderResource;

	// We have to support Metal for this object now
	check(IsMetalPlatform(GMaxRHIShaderPlatform));
	{
		if (!MetalTextureCache)
		{
            id<MTLDevice> Device = (__bridge id<MTLDevice>)GetIMetalDynamicRHI()->RHIGetDevice();
			check(Device);
			
			CVReturn Return = CVMetalTextureCacheCreate(kCFAllocatorDefault, nullptr, Device, nullptr, &MetalTextureCache);
			check(Return == kCVReturnSuccess);
		}
		check(MetalTextureCache);
		
		if (CVPixelBufferIsPlanar(Frame))
		{
			// Expecting BiPlanar kCVPixelFormatType_420YpCbCr8BiPlanar Full/Video
			check(CVPixelBufferGetPlaneCount(Frame) == 2);

			int32 YWidth = CVPixelBufferGetWidthOfPlane(Frame, 0);
			int32 YHeight = CVPixelBufferGetHeightOfPlane(Frame, 0);
			
			CVMetalTextureRef YTextureRef = nullptr;
			CVReturn Result = CVMetalTextureCacheCreateTextureFromImage(kCFAllocatorDefault, MetalTextureCache, Frame, nullptr, MTLPixelFormatR8Unorm, YWidth, YHeight, 0, &YTextureRef);
			check(Result == kCVReturnSuccess);
			check(YTextureRef);
			
			int32 UVWidth = CVPixelBufferGetWidthOfPlane(Frame, 1);
			int32 UVHeight = CVPixelBufferGetHeightOfPlane(Frame, 1);
			
			CVMetalTextureRef UVTextureRef = nullptr;
			Result = CVMetalTextureCacheCreateTextureFromImage(kCFAllocatorDefault, MetalTextureCache, Frame, nullptr, MTLPixelFormatRG8Unorm, UVWidth, UVHeight, 1, &UVTextureRef);
			check(Result == kCVReturnSuccess);
			check(UVTextureRef);
			
			// Metal can upload directly from an IOSurface to a 2D texture
			IMetalDynamicRHI* MetalDynamicRHI = GetIMetalDynamicRHI();
						
			TRefCountPtr<FRHITexture> YTex = MetalDynamicRHI->RHICreateTexture2DFromCVMetalTexture(PF_G8, ETextureCreateFlags::NoTiling | ETextureCreateFlags::ShaderResource, FClearValueBinding::White, YTextureRef);
			TRefCountPtr<FRHITexture> UVTex = MetalDynamicRHI->RHICreateTexture2DFromCVMetalTexture(PF_R8G8, ETextureCreateFlags::NoTiling | ETextureCreateFlags::ShaderResource, FClearValueBinding::White, UVTextureRef);

			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("Info"), YWidth, YHeight, PF_B8G8R8A8)
				.SetFlags(ETextureCreateFlags::NoTiling | ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::SRGB);
			
			ShaderResource = RHICmdList.CreateTexture(Desc);
			
			// render video frame into sink texture
			{
				// configure media shaders
				auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
				TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);
				TShaderMapRef<FYCbCrConvertPS> PixelShader(ShaderMap);
				
				FRHIRenderPassInfo RPInfo(ShaderResource, ERenderTargetActions::Load_Store);
				RHICmdList.BeginRenderPass(RPInfo, TEXT("AvfMediaSampler"));
				{
					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

					GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					SetShaderParametersLegacyPS(RHICmdList, PixelShader, YTex, UVTex, *ColorTransform, MediaShaders::YUVOffset8bits, true);

					FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);
					RHICmdList.SetStreamSource(0, VertexBuffer, 0);
					RHICmdList.SetViewport(0, 0, 0.0f, YWidth, YHeight, 1.0f);

					RHICmdList.DrawPrimitive(0, 2, 1);
				}
				RHICmdList.EndRenderPass();
				RHICmdList.Transition(FRHITransitionInfo(ShaderResource, ERHIAccess::RTV, ERHIAccess::SRVMask));
			}
			
			CFRelease(YTextureRef);
			CFRelease(UVTextureRef);
		}
		else
		{
			int32 Width = CVPixelBufferGetWidth(Frame);
			int32 Height = CVPixelBufferGetHeight(Frame);
			
			CVMetalTextureRef TextureRef = nullptr;
			CVReturn Result = CVMetalTextureCacheCreateTextureFromImage(kCFAllocatorDefault, MetalTextureCache, Frame, nullptr, MTLPixelFormatBGRA8Unorm_sRGB, Width, Height, 0, &TextureRef);
			check(Result == kCVReturnSuccess);
			check(TextureRef);

			IMetalDynamicRHI* MetalDynamicRHI = GetIMetalDynamicRHI();
						
			ShaderResource = MetalDynamicRHI->RHICreateTexture2DFromCVMetalTexture(PF_B8G8R8A8, ETextureCreateFlags::NoTiling | ETextureCreateFlags::ShaderResource, FClearValueBinding::White, TextureRef);
			
			CFRelease(TextureRef);
		}
	}
	
	if(ShaderResource.IsValid())
	{
		if (VideoSample->Initialize(ShaderResource, Dim, OutputDim, SampleTime, SampleDuration, Frame))
		{
			ProcessOutputSample(VideoSample);
		}
	}
	
#else //WITH_ENGINE
	
	if(CVPixelBufferLockBaseAddress(Frame, kCVPixelBufferLock_ReadOnly) == kCVReturnSuccess)
	{
		uint8* VideoData = (uint8*)CVPixelBufferGetBaseAddress(Frame);
		
		if (VideoSample->Initialize(VideoData, Dim, OutputDim, FrameStride, SampleTime, SampleDuration))
		{
			ProcessOutputSample(VideoSample);
		}
		
		
		CVPixelBufferUnlockBaseAddress(Frame, kCVPixelBufferLock_ReadOnly);
	}
	
#endif //WITH_ENGINE
	
	CVPixelBufferRelease(Frame);
}

void FAvfMediaVideoSampler::ProcessOutputSample(const TSharedRef<IMediaTextureSample, ESPMode::ThreadSafe>& Sample)
{
	Samples.AddVideo(Sample);
}
