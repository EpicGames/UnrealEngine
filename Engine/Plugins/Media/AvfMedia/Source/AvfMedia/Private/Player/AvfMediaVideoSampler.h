// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "Templates/SharedPointer.h"

#include "IMediaTextureSample.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import <AVFoundation/AVFoundation.h>

class FAvfMediaTextureSamplePool;
class FMediaSamples;


/**
 * Creates samples from video frames.
 */
class DLLEXPORT FAvfMediaVideoSampler
{
public:

	/** Default constructor. */
	FAvfMediaVideoSampler(FMediaSamples& InSamples);

	/** Virtual destructor. */
	virtual ~FAvfMediaVideoSampler();

public:

	/**
	 * Set the video output object to be sampled.
	 *
	 * This method must be called on the render thread.
	 *
	 * @param Output The output object.
	 */
	void SetOutput(AVPlayerItemVideoOutput* Output, float InFrameRate, bool bFullRange = false);

	/** Tick the video sampler (on the render thread). */
	void Tick();

	void ProcessFrame(CVPixelBufferRef Frame, FTimespan SampleTime, FTimespan SampleDuration);

	void Reset();
	
protected:

	virtual void ProcessOutputSample(const TSharedRef<IMediaTextureSample, ESPMode::ThreadSafe>& Sample);
private:

	/** Mutex to ensure thread-safe access */
	FCriticalSection CriticalSection;

	/** The track's video output handle */
	AVPlayerItemVideoOutput* Output;

	/** The media sample queue. */
	FMediaSamples& Samples;

	/** Video sample object pool. */
	FAvfMediaTextureSamplePool* VideoSamplePool;

	/** Current Video cached decode information */
	float FrameDuration;
	FMatrix const* ColorTransform;

#if WITH_ENGINE

	/** The Metal texture cache for unbuffered texture uploads. */
	CVMetalTextureCacheRef MetalTextureCache;

#endif
};
