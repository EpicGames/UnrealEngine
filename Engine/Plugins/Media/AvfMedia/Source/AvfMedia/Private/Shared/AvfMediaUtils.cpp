// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvfMediaUtils.h"

#include "HAL/PlatformProcess.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"


namespace AvfMedia
{
	FString CodecTypeToString(CMVideoCodecType CodecType)
	{
		if (CodecType == kCMVideoCodecType_422YpCbCr8) return TEXT("422YpCbCr8");
		if (CodecType == kCMVideoCodecType_Animation) return TEXT("Animation");
		if (CodecType == kCMVideoCodecType_Cinepak) return TEXT("Cinepak");
		if (CodecType == kCMVideoCodecType_JPEG) return TEXT("JPEG");
		if (CodecType == kCMVideoCodecType_JPEG_OpenDML) return TEXT("JPEG OpenDML");
		if (CodecType == kCMVideoCodecType_SorensonVideo) return TEXT("Sorenson Video");
		if (CodecType == kCMVideoCodecType_SorensonVideo3) return TEXT("Sorenson Video 3");
		if (CodecType == kCMVideoCodecType_H263) return TEXT("H263");
		if (CodecType == kCMVideoCodecType_H264) return TEXT("H264");
		if (CodecType == kCMVideoCodecType_HEVC) return TEXT("HEVC");
		if (CodecType == kCMVideoCodecType_MPEG4Video) return TEXT("MPEG4 Video");
		if (CodecType == kCMVideoCodecType_MPEG2Video) return TEXT("MPEG2 Video");
		if (CodecType == kCMVideoCodecType_MPEG1Video) return TEXT("MPEG1 Video");

		if (CodecType == kCMVideoCodecType_DVCNTSC) return TEXT("DVC NTSC");
		if (CodecType == kCMVideoCodecType_DVCPAL) return TEXT("DVC PAL");
		if (CodecType == kCMVideoCodecType_DVCProPAL) return TEXT("DVCPro PAL");
		if (CodecType == kCMVideoCodecType_DVCPro50NTSC) return TEXT("DVCPro50 NTSC");
		if (CodecType == kCMVideoCodecType_DVCPro50PAL) return TEXT("DVCPro50 PAL");
		if (CodecType == kCMVideoCodecType_DVCPROHD720p60) return TEXT("DVCPRO HD 720p 60");
		if (CodecType == kCMVideoCodecType_DVCPROHD720p50) return TEXT("DVCPRO HD 720p 50");
		if (CodecType == kCMVideoCodecType_DVCPROHD1080i60) return TEXT("DVCPRO HD 1080i 60");
		if (CodecType == kCMVideoCodecType_DVCPROHD1080i50) return TEXT("DVCPRO HD 1080i 50");
		if (CodecType == kCMVideoCodecType_DVCPROHD1080p30) return TEXT("DVCPRO HD 1080p 30");
		if (CodecType == kCMVideoCodecType_DVCPROHD1080p25) return TEXT("DVCPRO HD 1080p 25");

		if (CodecType == kCMVideoCodecType_AppleProRes4444) return TEXT("Apple ProRes 4444");
		if (CodecType == kCMVideoCodecType_AppleProRes422HQ) return TEXT("Apple ProRes 422 HQ");
		if (CodecType == kCMVideoCodecType_AppleProRes422) return TEXT("Apple ProRes 422");
		if (CodecType == kCMVideoCodecType_AppleProRes422LT) return TEXT("Apple ProRes 422 LT");
		if (CodecType == kCMVideoCodecType_AppleProRes422Proxy) return TEXT("Apple ProRes 422 Proxy");

		return TEXT("Unknown");
	}


	FString MediaTypeToString(NSString* MediaType)
	{
		if ([MediaType isEqualToString : AVMediaTypeAudio])
		{
			return TEXT("Audio");
		}
		
		if ([MediaType isEqualToString : AVMediaTypeClosedCaption])
		{
			return TEXT("Closed Caption");
		}
		
		if ([MediaType isEqualToString : AVMediaTypeSubtitle])
		{
			return TEXT("Subtitle");
		}
		
		if ([MediaType isEqualToString : AVMediaTypeText])
		{
			return TEXT("Text");
		}
		
		if ([MediaType isEqualToString : AVMediaTypeTimecode])
		{
			return TEXT("Timecode (unsupported)");
		}
		
		if ([MediaType isEqualToString : AVMediaTypeVideo])
		{
			return TEXT("Video");
		}
		
		return TEXT("Unknown");
	}


#if !PLATFORM_MAC
	FString ConvertToIOSPath(const FString& Filename, bool bForWrite)
	{
		FString Result = Filename;

		if (Result.Contains(TEXT("/OnDemandResources/")))
		{
			return Result;
		}

		Result.ReplaceInline(TEXT("../"), TEXT(""));
		Result.ReplaceInline(TEXT(".."), TEXT(""));
		Result.ReplaceInline(FPlatformProcess::BaseDir(), TEXT(""));

		if (bForWrite)
		{
#if FILESHARING_ENABLED
			static FString WritePathBase = FString([NSSearchPathForDirectoriesInDomains(NSLibraryDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
#else
			static FString WritePathBase = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
#endif
			return WritePathBase + Result;
		}
		else
		{
			// if 'filehostip' exists in the command line, cook on the fly read path should be used
			FString Value;

			// Cache this value as the command line doesn't change...
			static bool bHasHostIP = FParse::Value(FCommandLine::Get(), TEXT("filehostip"), Value) || FParse::Value(FCommandLine::Get(), TEXT("streaminghostip"), Value);
			static bool bIsIterative = FParse::Value(FCommandLine::Get(), TEXT("iterative"), Value);
			
			if (bHasHostIP)
			{
				static FString ReadPathBase = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
				return ReadPathBase + Result;
			}
			
			if (bIsIterative)
			{
				static FString ReadPathBase = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
				return ReadPathBase + Result.ToLower();
			}
			
			static FString ReadPathBase = FString([[NSBundle mainBundle] bundlePath]) + TEXT("/cookeddata/");
			return ReadPathBase + Result.ToLower();
		}
	}
#endif
}
