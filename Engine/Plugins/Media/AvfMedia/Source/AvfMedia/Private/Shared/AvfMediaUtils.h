// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import <AVFoundation/AVFoundation.h>


namespace AvfMedia
{
	/**
	 * Convert the given codec type to a human readable string.
	 *
	 * @param CodecType The codec type to convert.
	 * @return The codec type's strings representation.
	 */
	FString CodecTypeToString(CMVideoCodecType CodecType);

	/**
	 * Convert the given media type to a human readable string.
	 *
	 * @param MediaType The media type to convert.
	 * @return The media type's strings representation.
	 */
	FString MediaTypeToString(NSString* MediaType);

#if !PLATFORM_MAC
	/**
	 * Convert the given file name to an iOS compatible file path.
	 *
	 * @param Filename The file name to convert.
	 * @param bForWrite
	 * @return The iOS path.
	 */
	FString ConvertToIOSPath(const FString& Filename, bool bForWrite);
#endif
}
