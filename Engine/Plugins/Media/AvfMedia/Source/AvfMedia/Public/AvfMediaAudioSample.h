// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "IMediaAudioSample.h"
#include "MediaObjectPool.h"
#include "Misc/Timespan.h"


/**
 * Audio sample generated by AvfMedia player.
 */
class FAvfMediaAudioSample
	: public IMediaAudioSample
	, public IMediaPoolable
{
public:

	/** Default constructor. */
	FAvfMediaAudioSample()
		: Channels(0)
		, Duration(0)
		, Frames(0)
		, SampleRate(0)
		, Time(FTimespan::Zero())
		, Format(EMediaAudioSampleFormat::Undefined)
	{ }

	/** Virtual destructor. */
	virtual ~FAvfMediaAudioSample() = default;

public:

	/**
	 * Get a writable pointer to the sample buffer.
	 *
	 * @return Sample buffer.
	 */
	uint8* GetMutableBuffer()
	{
		return Buffer.GetData();
	}

	/**
	 * Initialize the sample.
	 *
	 * @param InBufferSize The size of the audio buffer.
	 * @param InFrames Number of frames in the buffer.
	 * @param InChannels Number of audio channels.
	 * @param InSampleRate The sample rate.
	 * @param InTime The sample time (in the player's local clock).
	 * @return true on success, false otherwise.
	 */
	bool Initialize(
		uint32 InBufferSize,
		uint32 InFrames,
		uint32 InChannels,
		uint32 InSampleRate,
		FTimespan InTime,
		FTimespan InDuration,
		EMediaAudioSampleFormat InFormat = EMediaAudioSampleFormat::Float)
	{
		if (InBufferSize == 0)
		{
			return false;
		}

		Buffer.Reset(InBufferSize);
		Buffer.AddUninitialized(InBufferSize);

		Channels = InChannels;
		Frames = InFrames;
		SampleRate = InSampleRate;
		Time = InTime;
		Duration = InDuration;
		Format = InFormat;

		return true;
	}

public:

	//~ IMediaAudioSample interface

	virtual const void* GetBuffer() override
	{
		return Buffer.GetData();
	}

	virtual uint32 GetChannels() const override
	{
		return Channels;
	}

	virtual FTimespan GetDuration() const override
	{
		return Duration;
	}

	virtual EMediaAudioSampleFormat GetFormat() const override
	{
		return Format;
	}

	virtual uint32 GetFrames() const override
	{
		return Frames;
	}

	virtual uint32 GetSampleRate() const override
	{
		return SampleRate;
	}

	virtual FMediaTimeStamp GetTime() const override
	{
		return FMediaTimeStamp(Time);
	}

private:

	/** The sample's frame buffer. */
	TArray<uint8> Buffer;

	/** Number of audio channels. */
	uint32 Channels;

	/** Duration for which the sample is valid. */
	FTimespan Duration;

	/** Number of frames in the buffer. */
	uint32 Frames;

	/** Audio sample rate (in samples per second). */
	uint32 SampleRate;

	/** Play time for which the sample was generated. */
	FTimespan Time;
	
	/** Sample Format  (8/16/24/32 Integer or 32 Float) */
	EMediaAudioSampleFormat Format;
};


/** Implements a pool for AVF audio sample objects. */
class FAvfMediaAudioSamplePool : public TMediaObjectPool<FAvfMediaAudioSample> { };
