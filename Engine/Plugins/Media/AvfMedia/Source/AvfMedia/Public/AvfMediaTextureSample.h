// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IMediaTextureSample.h"
#include "MediaObjectPool.h"
#include "Math/IntPoint.h"
#include "Misc/Timespan.h"

#if WITH_ENGINE
	#include "RHI.h"
#else
	#include "Containers/Array.h"
#endif

/** CVPixelBuffer.h on macOS pulls in CoreServices -> CarbonCore -> NumberFormatting.h
 * which defines its own struct FVector, clashing with UE's type alias. Temporarily
 * redirect FVector to a harmless name while the Apple header is parsed.
 */
#if PLATFORM_MAC
	#pragma push_macro("FVector")
	#define FVector FVectorWorkaround
#endif
#include <CoreVideo/CVPixelBuffer.h>
#if PLATFORM_MAC
	#undef FVector
	#pragma pop_macro("FVector")
#endif

/**
 * Texture sample generated by AvfMedia player.
 */
class DLLEXPORT FAvfMediaTextureSample
	: public IMediaTextureSample
	, public IMediaPoolable
{
public:

	/** Virtual destructor. */
	virtual ~FAvfMediaTextureSample();

public:

#if WITH_ENGINE

	/**
	 * Initialize the texture sample with a texture resource.
	 *
	 * @param InTexture The texture resource containing the video frame.
	 * @param InDim The sample buffer's width and height (in pixels).
	 * @param InOutputDim The sample's output width and height (in pixels).
	 * @param InTime The sample time (in the player's local clock).
	 * @param InDuration The duration for which the sample is valid.
	 * @param InPixelBuffer Optional CVPixelBuffer to retain for CPU access via GetBuffer().
	 * @return true on success, false otherwise.
	 */
	bool Initialize(
		TRefCountPtr<FRHITexture>& InTexture,
		const FIntPoint& InDim,
		const FIntPoint& InOutputDim,
		FTimespan InTime,
		FTimespan InDuration,
		CVPixelBufferRef InPixelBuffer = nullptr);

#else

	/**
	 * Initialize the texture sample with a memory buffer.
	 *
	 * @param InBuffer The memory buffer containing the video frame.
	 * @param InDim The sample buffer's width and height (in pixels).
	 * @param InOutputDim The sample's output width and height (in pixels).
	 * @param InStride Number of bytes per pixel row.
	 * @param InTime The sample time (in the player's local clock).
	 * @param InDuration The duration for which the sample is valid.
	 * @return true on success, false otherwise.
	 */
	bool Initialize(
		uint8* InBuffer,
		const FIntPoint& InDim,
		const FIntPoint& InOutputDim,
		uint32 InStride,
		FTimespan InTime,
		FTimespan InDuration);

#endif

public:

	//~ IMediaTextureSample interface

	virtual const void* GetBuffer() override;
	virtual FIntPoint GetDim() const override;
	virtual FTimespan GetDuration() const override;
	virtual EMediaTextureSampleFormat GetFormat() const override;
	virtual FIntPoint GetOutputDim() const override;
	virtual uint32 GetStride() const override;
	virtual FRHITexture* GetTexture() const override;
	virtual FMediaTimeStamp GetTime() const override;
	virtual bool IsCacheable() const override;
	virtual bool IsOutputSrgb() const override;
	virtual void Reset() override;

	//~ IMediaPoolable interface

	virtual void ShutdownPoolable() override;

private:

	void UnlockAndReleasePixelBuffer();

	/** Width and height of the texture sample. */
	FIntPoint Dim;

	/** Duration for which the sample is valid. */
	FTimespan Duration;

	/** Width and height of the output. */
	FIntPoint OutputDim;

	/** Sample time. */
	FTimespan Time;

#if WITH_ENGINE
	/** The sample's texture resource. */
	TRefCountPtr<FRHITexture> Texture;

#else
	/** The sample's data buffer. */
	TArray<uint8> Buffer;

	/** Number of bytes per pixel row. */
	uint32 Stride;

#endif

	/** Retained CVPixelBuffer for CPU access (locked read-only while sample is active). */
	CVPixelBufferRef LockedPixelBuffer = nullptr;
};


/** Implements a pool for AVF texture sample objects. */
class FAvfMediaTextureSamplePool : public TMediaObjectPool<FAvfMediaTextureSample> { };
