// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AvfMediaCapture : ModuleRules
	{
		public AvfMediaCapture(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"Engine",
					"ApplicationCore",
					"MediaUtils",
					"RenderCore",
					"RHI",
					"MetalRHI",
					"AvfMedia"
				});

			AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
			
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
				});


			PublicFrameworks.AddRange(
				new string[] {
					"CoreMedia",
					"CoreVideo",
					"AVFoundation",
					"AudioToolbox",
					"MediaToolbox",
					"QuartzCore"
				});
		}
	}
}
