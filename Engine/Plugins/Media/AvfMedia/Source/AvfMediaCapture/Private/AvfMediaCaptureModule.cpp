// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvfMediaCapturePrivate.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "IMediaModule.h"
#include "IMediaPlayerFactory.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"
#include "IMediaCaptureSupport.h"
#include "Player/AvfMediaCapturePlayer.h"

#if WITH_EDITOR
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
	#include "UObject/Class.h"
	#include "UObject/WeakObjectPtr.h"
#endif

#include "Apple/ScopeAutoreleasePool.h"

#import <AVFoundation/AVFoundation.h>


DEFINE_LOG_CATEGORY(LogAvfMediaCapture);

#define LOCTEXT_NAMESPACE "FAvfMediaCaptureFactoryModule"

#if (PLATFORM_IOS && (defined(__IPHONE_17_0) && __IPHONE_OS_VERSION_MAX_ALLOWED >= __IPHONE_17_0))
	#define IOS_17_APIS_AVAILABLE 1
#else
	#define IOS_17_APIS_AVAILABLE 0
#endif

#if (PLATFORM_MAC && (defined(__MAC_14_0) && __MAC_OS_X_VERSION_MAX_ALLOWED >= __MAC_14_0))
	#define MAC_14_APIS_AVAILABLE 1
#else
	#define MAC_14_APIS_AVAILABLE 0
#endif

// New AVCaptureDeviceType APIs don't compile on old IOS or MAC SDKs
#if (IOS_17_APIS_AVAILABLE || MAC_14_APIS_AVAILABLE)
	#define USE_NEW_CAPTURE_DEVICE_TYPE_API 1
#else
	#define USE_NEW_CAPTURE_DEVICE_TYPE_API 0
#endif

/**
 * Implements the AvfMediaCapture module.
 */
class FAvfMediaCaptureModule
	: public IModuleInterface
	, public IMediaPlayerFactory
	, public IMediaCaptureSupport
{
public:
	void EnumerateCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos, EMediaCaptureDeviceType TargetDeviceType)
	{
		SCOPED_AUTORELEASE_POOL

        NSMutableArray* DeviceTypes = [[NSMutableArray alloc] init];

		FString Scheme;
		AVMediaType MediaType;
		if (TargetDeviceType == EMediaCaptureDeviceType::Audio)
		{
			Scheme = TEXT("audcap://");
			MediaType = AVMediaTypeAudio;
			#if USE_NEW_CAPTURE_DEVICE_TYPE_API
				[DeviceTypes addObject: AVCaptureDeviceTypeMicrophone];
			#else
				[DeviceTypes addObject: AVCaptureDeviceTypeBuiltInMicrophone];
			#endif
		}
		else if (TargetDeviceType == EMediaCaptureDeviceType::Video)
		{
			Scheme = TEXT("vidcap://");
			MediaType = AVMediaTypeVideo;
			[DeviceTypes addObject: AVCaptureDeviceTypeBuiltInWideAngleCamera];

			#if PLATFORM_IOS
				[DeviceTypes addObject: AVCaptureDeviceTypeBuiltInUltraWideCamera];
				[DeviceTypes addObject: AVCaptureDeviceTypeBuiltInTelephotoCamera];
			#endif

			#if USE_NEW_CAPTURE_DEVICE_TYPE_API
				[DeviceTypes addObject: AVCaptureDeviceTypeExternal];
			#elif PLATFORM_MAC
				// AVCaptureDeviceTypeExternalUnknown is only available on macOS 10.15 - 14.0
				// https://developer.apple.com/documentation/avfoundation/avcapturedevicetypeexternalunknown?language=objc
				[DeviceTypes addObject: AVCaptureDeviceTypeExternalUnknown];
			#endif
		}

		if ([DeviceTypes count] == 0)
		{
			return;
		}

		AVCaptureDeviceDiscoverySession* LocalDiscoverySession = [AVCaptureDeviceDiscoverySession discoverySessionWithDeviceTypes: DeviceTypes
			                                                      mediaType: nil
			                                                      position: AVCaptureDevicePositionUnspecified];
		if (LocalDiscoverySession != nil)
		{
			NSArray<AVCaptureDevice*>* Devices = LocalDiscoverySession.devices;
			for(uint32 i = 0; i < Devices.count; ++i)
			{
				AVCaptureDevice* AvailableDevice = Devices[i];

				// It's not clear whether external media types will be limited to video.
				// In which case we double check that the detected device supports the target media type.
				if (![AvailableDevice hasMediaType: MediaType])
				{
					continue;
				}

				FMediaCaptureDeviceInfo DeviceInfo;

				DeviceInfo.Type = TargetDeviceType;
				DeviceInfo.DisplayName = FText::FromString(FString(AvailableDevice.localizedName));
				DeviceInfo.Url = Scheme + FString(AvailableDevice.uniqueID);
				DeviceInfo.Info = FString(AvailableDevice.manufacturer);

				OutDeviceInfos.Add(MoveTemp(DeviceInfo));
			}
		}
	}

	//~ IMediaCaptureSupport interface

	virtual void EnumerateAudioCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos)
	{
		EnumerateCaptureDevices(OutDeviceInfos, EMediaCaptureDeviceType::Audio);
	}

	virtual void EnumerateVideoCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos)
	{
		EnumerateCaptureDevices(OutDeviceInfos, EMediaCaptureDeviceType::Video);
	}

	//~ IMediaPlayerFactory interface

	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
	}

	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		FString Scheme;
		FString Location;

		// check scheme
		if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
			}

			return 0;
		}

		if (!SupportedUriSchemes.Contains(Scheme))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
			}

			return 0;
		}

		return 100;
	}

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		return MakeShared<FAvfMediaCapturePlayer, ESPMode::ThreadSafe>(EventSink);
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("MediaCaptureDisplayName", "Apple AV Foundation");
	}

	virtual FName GetPlayerName() const override
	{
		static FName PlayerName(TEXT("AvfMediaCapture"));
		return PlayerName;
	}

	virtual FGuid GetPlayerPluginGUID() const override
	{
		static FGuid PlayerPluginGUID(0xcf78bfd2, 0x0c1111ed, 0x861d0242, 0xac120002);
		return PlayerPluginGUID;
	}

	virtual const TArray<FString>& GetSupportedPlatforms() const override
	{
		return SupportedPlatforms;
	}

	virtual bool SupportsFeature(EMediaFeature Feature) const override
	{
		return ((Feature == EMediaFeature::AudioSamples) ||
				(Feature == EMediaFeature::VideoSamples));
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// supported schemes
		SupportedUriSchemes.Add(TEXT("vidcap"));
		SupportedUriSchemes.Add(TEXT("audcap"));

		// supported platforms
		SupportedPlatforms.Add(TEXT("Mac"));
		SupportedPlatforms.Add(TEXT("iOS"));

		// register factory support functions
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");
		if (MediaModule != nullptr)
		{
			MediaModule->RegisterPlayerFactory(*this);
			MediaModule->RegisterCaptureSupport(*this);
		}
	}

	virtual void ShutdownModule() override
	{
		// unregister factory support functions
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");
		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterPlayerFactory(*this);
			MediaModule->UnregisterCaptureSupport(*this);
		}
	}

private:

	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAvfMediaCaptureModule, AvfMediaCapture);
