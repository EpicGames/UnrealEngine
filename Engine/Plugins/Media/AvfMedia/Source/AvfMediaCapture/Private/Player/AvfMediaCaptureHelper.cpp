// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvfMediaCaptureHelper.h"

// Steps to try if permissions are not working correctly in macOS

// 1) macOS resetting permissions, also useful for testing:
//  tccutil reset Camera
//  tccutil reset Microphone
//
// 2) System integrity protection has to be enabled for permissions to work correctly, check using:
//  csrutil status
//
// 3) delete permissions database then reboot mac:
//  ~/Library/Application\\ Support/com.apple.TCC

@interface AvfMediaCaptureHelper()<AVCaptureVideoDataOutputSampleBufferDelegate, AVCaptureAudioDataOutputSampleBufferDelegate>

@property (nonatomic, assign) AVMediaType mediaType;
@property (nonatomic, assign) AVCaptureSession* captureSession;
@property (nonatomic, assign) AVCaptureDevice* captureDevice;
@property (nonatomic, assign) AVCaptureDeviceInput*	deviceInput;
@property (nonatomic, assign) AVCaptureOutput* captureOutput;
@property (nonatomic, assign) dispatch_queue_t callBackQueue;

@property (nonatomic, assign) void (^sampleBufferCallback)(CMSampleBufferRef);
@property (nonatomic, assign) void (^notificationCallback)(NSNotification* const Notification);

@end

@implementation AvfMediaCaptureHelper

- (instancetype)init:(AVMediaType)mediaType
{
	self = [super init];
	if (self != nil)
	{
		self.mediaType = mediaType;
		self.captureSession = nil;
		self.captureDevice = nil;
		self.deviceInput = nil;
		self.captureOutput = nil;
		self.sampleBufferCallback = nil;
		self.notificationCallback = nil;
		self.callBackQueue = dispatch_queue_create("AvfCaptureSessionSampleCallbackQueue", DISPATCH_QUEUE_SERIAL);
	}
	return self;
}

- (void)dealloc
{
	[self reset];
	
	if (self.callBackQueue != nil)
	{
		dispatch_release(self.callBackQueue);
		self.callBackQueue = nil;
	}
	
	[super dealloc];
}

- (void)reset
{
	if (self.captureSession.inputs.count > 0 && self.captureSession.outputs.count > 0)
	{
		[[NSNotificationCenter defaultCenter] removeObserver:self];
	}
	
	if (self.captureSession.isRunning)
	{
		[self.captureSession stopRunning];
	}
	
	if (self.captureOutput != nil)
	{
		[self.captureSession removeOutput:self.captureOutput];
		[self.captureOutput release];
		self.captureOutput = nil;
	}
	
	if (self.deviceInput != nil)
	{
		[self.captureSession removeInput:self.deviceInput];
		[self.deviceInput release];
		self.deviceInput = nil;
	}
	
	if (self.captureSession != nil)
	{
		[self.captureSession release];
		self.captureSession = nil;
	}
	
	if (self.captureDevice != nil)
	{
		[self.captureDevice release];
		self.captureDevice = nil;
	}
	
	if (self.sampleBufferCallback != nil)
	{
		Block_release(self.sampleBufferCallback);
		self.sampleBufferCallback = nil;
	}
	
	if (self.notificationCallback != nil)
	{
		Block_release(self.notificationCallback);
		self.notificationCallback = nil;
	}
}

+ (EAvfMediaCaptureAuthStatus)authorizationStatusForMediaType:(AVMediaType)mediaType
{
	if (mediaType != AVMediaTypeVideo && mediaType != AVMediaTypeAudio)
	{
		return EAvfMediaCaptureAuthStatus::InvalidRequest;
	}

	NSDictionary<NSString*,id>* infoDictionary = [NSBundle mainBundle].infoDictionary;
	
	id entry = nil;
	
	if (infoDictionary != nil)
	{
		if (mediaType == AVMediaTypeVideo)
		{
			entry = infoDictionary[@"NSCameraUsageDescription"];
		}
		else if (mediaType == AVMediaTypeAudio)
		{
			entry = infoDictionary[@"NSMicrophoneUsageDescription"];
		}
	}
	
	if (entry == nil)
	{
		return EAvfMediaCaptureAuthStatus::MissingInfoPListEntry;
	}
	
	return (EAvfMediaCaptureAuthStatus)[AVCaptureDevice authorizationStatusForMediaType:mediaType];
}

+ (EAvfMediaCaptureAuthStatus)requestAccessForMediaType:(AVMediaType)mediaType completionCallback:(void (^)(EAvfMediaCaptureAuthStatus AuthStatus))cbHandler
{
	// Don't make request if the correct device access key is not in the info.plist otherwise the OS will terminate this app
	EAvfMediaCaptureAuthStatus authStatus = [AvfMediaCaptureHelper authorizationStatusForMediaType:mediaType];
	switch(authStatus)
	{
		case EAvfMediaCaptureAuthStatus::NotDetermined:
		{
			[AVCaptureDevice requestAccessForMediaType:mediaType completionHandler:^(BOOL bGranted)
			{
				cbHandler((EAvfMediaCaptureAuthStatus)[AVCaptureDevice authorizationStatusForMediaType:mediaType]);
			}];
			break;
		}
		case EAvfMediaCaptureAuthStatus::Authorized:
		case EAvfMediaCaptureAuthStatus::Restricted:
		case EAvfMediaCaptureAuthStatus::Denied:
		case EAvfMediaCaptureAuthStatus::MissingInfoPListEntry:
		case EAvfMediaCaptureAuthStatus::InvalidRequest:
		{
			break;
		}
	}
	
	return authStatus;
}

- (BOOL)setupCaptureSession:(NSString*)deviceID sampleBufferCallback:(void(^)(CMSampleBufferRef sampleBuffer))sampleCallbackBlock notificationCallback:(void(^)(NSNotification* const notification))notificationCallbackBlock
{
	[self reset];
	
	if (sampleCallbackBlock != nil)
	{
		self.sampleBufferCallback = Block_copy(sampleCallbackBlock);
	}
	if (notificationCallbackBlock != nil)
	{
		self.notificationCallback = Block_copy(notificationCallbackBlock);
	}
	
	self.captureDevice = [[AVCaptureDevice deviceWithUniqueID: deviceID] retain];

	// Check that the chosen device supports the specified media type
	if (![self.captureDevice hasMediaType: self.mediaType])
	{
		return NO;
	}
	
	if (self.captureDevice != nil)
	{
		self.captureSession = [[AVCaptureSession alloc] init];
		
		// Default to high so it starts with a reasonably good initial setting
		self.captureSession.sessionPreset = AVCaptureSessionPresetHigh;
		
		// Add device input
		{
			self.deviceInput = [[AVCaptureDeviceInput alloc] initWithDevice:self.captureDevice error:nil];
			
			if (self.deviceInput != nil && [self.captureSession canAddInput:self.deviceInput])
			{
				[self.captureSession addInput:self.deviceInput];
			}
		}
		
		// If we have an input add output capture
		if (self.captureSession.inputs.count > 0)
		{
            // Video or Audio Device - need the correct output
            if (self.mediaType == AVMediaTypeVideo)
			{
				AVCaptureVideoDataOutput* videoOutput = [[AVCaptureVideoDataOutput alloc] init];
				
				[videoOutput setSampleBufferDelegate:self queue:self.callBackQueue];
				videoOutput.alwaysDiscardsLateVideoFrames = YES;
				
				videoOutput.videoSettings = @{ 	(id)kCVPixelBufferPixelFormatTypeKey 	: @(kCVPixelFormatType_32BGRA),
												(id)kCVPixelBufferMetalCompatibilityKey : @(1)};
				
				self.captureOutput = videoOutput;
			}
			else
			{
				AVCaptureAudioDataOutput* audioOutput = [[AVCaptureAudioDataOutput alloc] init];
				
				[audioOutput setSampleBufferDelegate:self queue:self.callBackQueue];

#define MACOS_AUDIO_CAPTURE_USR_SETTINGS 0
#if PLATFORM_MAC && MACOS_AUDIO_CAPTURE_USR_SETTINGS != 0
				// AVCaptureAudioDataOutput.audioSettings is available on macOS only
				// Use default device / OS output instead.  This is left here for debug and test purposes
				// Changes the output audio format delivered to the sample buffer delete function
				// See AVFAudio/AVAudioSettings.h for available keys / values
				audioOutput.audioSettings = @{
												// LPCM=Linear Quantization Levels
												AVFormatIDKey					:@(kAudioFormatLinearPCM),
												// 8/16.24/32=INT, 32=FLOAT
												AVLinearPCMBitDepthKey 			: @(32),
												// NO=INT, YES=FLOAT, engine perfers float and will convert to float eventually what ever we output
												AVLinearPCMIsFloatKey 			: @YES,
												// Non Interleaved each channel is sequential one after each other e.g 3 channel samples = 111222333, Interleaved e.g. 123123123, engine wants interleaved
												AVLinearPCMIsNonInterleavedKey	: @NO,
												// As Per hardware
												AVNumberOfChannelsKey			: @(1)
											};
#endif
#undef MACOS_AUDIO_CAPTURE_USR_SETTINGS

				self.captureOutput = audioOutput;
			}
			
			if (self.captureOutput != nil && [self.captureSession canAddOutput:self.captureOutput])
			{
				[self.captureSession addOutput:self.captureOutput];
			}
		}
	}
	
	if (self.captureSession.inputs.count > 0 && self.captureSession.outputs.count > 0)
	{
		NSNotificationCenter* sharedCentre = [NSNotificationCenter defaultCenter];

		[sharedCentre addObserver:self selector:@selector(captureNotification:) name:AVCaptureSessionDidStartRunningNotification object:self.captureSession];
		[sharedCentre addObserver:self selector:@selector(captureNotification:) name:AVCaptureSessionDidStopRunningNotification object:self.captureSession];
		[sharedCentre addObserver:self selector:@selector(captureNotification:) name:AVCaptureSessionWasInterruptedNotification object:self.captureSession];
		[sharedCentre addObserver:self selector:@selector(captureNotification:) name:AVCaptureSessionInterruptionEndedNotification object:self.captureSession];
		[sharedCentre addObserver:self selector:@selector(captureNotification:) name:AVCaptureSessionRuntimeErrorNotification object:self.captureSession];
		
#if PLATFORM_MAC && WITH_EDITOR
		if ([self.captureOutput isKindOfClass:[AVCaptureVideoDataOutput class]])
		{
			[sharedCentre addObserver:self selector:@selector(captureNotification:) name:NSApplicationDidBecomeActiveNotification object:nil];
			[sharedCentre addObserver:self selector:@selector(captureNotification:) name:NSApplicationWillResignActiveNotification object:nil];
		}
#endif

		return YES;
	}

	return NO;
}

- (void)stopCaptureSession
{
	if (self.captureSession.isRunning)
	{
		[self.captureSession stopRunning];
	}
}

- (void)startCaptureSession
{
	if (!self.captureSession.isRunning)
	{
		[self.captureSession startRunning];
	}
}

- (BOOL)isCaptureRunning
{
	return self.captureSession.isRunning;
}

- (void)captureOutput:(AVCaptureOutput *)output didOutputSampleBuffer:(CMSampleBufferRef)sampleBuffer fromConnection:(AVCaptureConnection*)connection
{
	if (self.sampleBufferCallback != nil && sampleBuffer != NULL)
	{
		self.sampleBufferCallback(sampleBuffer);
	}
}

- (void)captureNotification:(NSNotification*)notification
{
	if (self.notificationCallback != nil && notification != nil)
	{
		self.notificationCallback(notification);
	}
}

- (AVMediaType)getCaptureDeviceMediaType
{
	return self.mediaType;
}

- (NSString*)getCaptureDeviceName
{
	return self.captureDevice.localizedName;
}

- (NSArray<AVCaptureDeviceFormat*>*)getCaptureDeviceAvailableFormats
{
	AVMediaType helperMediaType = self.mediaType;
	NSArray<AVCaptureDeviceFormat*>* filtered = [self.captureDevice.formats filteredArrayUsingPredicate: [NSPredicate predicateWithBlock:^BOOL(AVCaptureDeviceFormat* format, NSDictionary* bindings)
	{
		return [format.mediaType isEqualToString: helperMediaType];
	}]];

	return filtered;
}

- (NSInteger)getCaptureDeviceActiveFormatIndex
{
	NSArray<AVCaptureDeviceFormat*>* formats = [self getCaptureDeviceAvailableFormats];
	for (NSInteger formatIndex = 0; formatIndex < formats.count; ++formatIndex)
	{
		if ([formats[formatIndex] isEqual:self.captureDevice.activeFormat])
		{
			return formatIndex;
		}
	}
	return INDEX_NONE;
}

- (BOOL)setCaptureDeviceActiveFormatIndex:(NSInteger)formatIdx
{
	NSArray<AVCaptureDeviceFormat*>* formats = [self getCaptureDeviceAvailableFormats];
	if (formatIdx >= 0 && formatIdx < formats.count && [self.captureDevice lockForConfiguration: nil])
	{
		self.captureDevice.activeFormat = formats[formatIdx];
		[self.captureDevice unlockForConfiguration];
		return YES;
	}
	return NO;
}

@end
