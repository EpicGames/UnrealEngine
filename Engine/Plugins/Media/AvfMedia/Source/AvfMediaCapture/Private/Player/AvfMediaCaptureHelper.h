// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#include "Apple/PreAppleSystemHeaders.h"
#include <AVFoundation/AVFoundation.h>
#if PLATFORM_MAC
	#include <AppKit/AppKit.h>
#endif
#include "Apple/PostAppleSystemHeaders.h"

enum class EAvfMediaCaptureAuthStatus : uint32
{
	// Mapped Apple AVFoundation values
    NotDetermined	= AVAuthorizationStatusNotDetermined,
	Restricted		= AVAuthorizationStatusRestricted,
	Denied			= AVAuthorizationStatusDenied,
	Authorized		= AVAuthorizationStatusAuthorized,
    
    // Extra Error values
	MissingInfoPListEntry,
	InvalidRequest
};

@interface AvfMediaCaptureHelper : NSObject

// Returns current status or error
+ (EAvfMediaCaptureAuthStatus)authorizationStatusForMediaType:(AVMediaType)mediaType;

// Returns current status or error and requests access if not determined
+ (EAvfMediaCaptureAuthStatus)requestAccessForMediaType:(AVMediaType)mediaType completionCallback:(void(^)(EAvfMediaCaptureAuthStatus AuthStatus))cbHandler;

- (instancetype)init:(AVMediaType)mediaType;

- (BOOL)setupCaptureSession:(NSString*)deviceID
        sampleBufferCallback:(void(^)(CMSampleBufferRef sampleBuffer))sampleCallbackBlock
        notificationCallback:(void(^)(NSNotification* const notification))notificationCallbackBlock;
- (void)stopCaptureSession;
- (void)startCaptureSession;
- (BOOL)isCaptureRunning;

- (NSString*)getCaptureDeviceName;
- (AVMediaType)getCaptureDeviceMediaType;

- (NSArray<AVCaptureDeviceFormat*>*)getCaptureDeviceAvailableFormats;

- (NSInteger)getCaptureDeviceActiveFormatIndex;
- (BOOL)setCaptureDeviceActiveFormatIndex:(NSInteger)formatIdx;

@end
