// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvfMediaCapturePlayer.h"
#include "MediaSamples.h"
#include "Misc/MessageDialog.h"

#include "IMetalDynamicRHI.h"
#include "Apple/AppleStringUtils.h"
#include "Apple/ScopeAutoreleasePool.h"

inline FString FStringFromAppleFourCharWordCode(uint32 Word)
{
	char TypeCode[5];
	{
		TypeCode[0] = ((char*)&Word)[3];
		TypeCode[1] = ((char*)&Word)[2];
		TypeCode[2] = ((char*)&Word)[1];
		TypeCode[3] = ((char*)&Word)[0];
		TypeCode[4] = 0;
	}
	return FString((char*)(&TypeCode));
}

template<class T>
void InterleaveCopy(void* DestPtr, void* SrcPtr, size_t NumChannels, size_t NumFrames, size_t SrcFrameOffset, size_t SrcNumFrames)
{
	T* DestPtrType = (T*)DestPtr;
	T* SrcPtrType = (T*)SrcPtr;

	for(size_t SrcFrameIdx = 0;SrcFrameIdx < SrcNumFrames;++SrcFrameIdx)
	{
		size_t GlobalSrcFrameIdx = SrcFrameOffset + SrcFrameIdx;
		size_t ChannelIdx = GlobalSrcFrameIdx / NumFrames;
		size_t DestFrameIdx = (NumChannels * (GlobalSrcFrameIdx % NumFrames)) + ChannelIdx;

		DestPtrType[DestFrameIdx] = SrcPtrType[SrcFrameIdx];
	}
}

FAvfMediaCapturePlayer::FAvfMediaCapturePlayer(IMediaEventSink& InEventSink)
: EventSink(InEventSink)
, MediaSamples(MakeShared<FMediaSamples, ESPMode::ThreadSafe>())
, MediaCaptureHelper(nil)
, MetalTextureCache(NULL)
, CurrentRate(0.f)
, CurrentTime(FTimespan::MinValue())
#if PLATFORM_MAC && WITH_EDITOR
, ThrottleDuration(0.0)
, LastConsumedTimeStamp(0.0)
#endif
{}

FAvfMediaCapturePlayer::~FAvfMediaCapturePlayer()
{
	Close();
	
	if(MetalTextureCache != NULL)
	{
		CFRelease(MetalTextureCache);
		MetalTextureCache = NULL;
	}
}

// IMediaPlayer Interface

IMediaCache& FAvfMediaCapturePlayer::GetCache()
{
	return *this;
}

IMediaControls& FAvfMediaCapturePlayer::GetControls()
{
	return *this;
}

FString FAvfMediaCapturePlayer::GetInfo() const
{
	return FString("");
}

FGuid FAvfMediaCapturePlayer::GetPlayerPluginGUID() const
{
	static FGuid PlayerPluginGUID(0xcf78bfd2, 0x0c1111ed, 0x861d0242, 0xac120002);
	return PlayerPluginGUID;
}

IMediaSamples& FAvfMediaCapturePlayer::GetSamples()
{
	return *MediaSamples.Get();
}

FString FAvfMediaCapturePlayer::GetStats() const
{
	return FString("");
}

IMediaTracks& FAvfMediaCapturePlayer::GetTracks()
{
	return *this;
}

FString FAvfMediaCapturePlayer::GetUrl() const
{
	return URL;
}

FText FAvfMediaCapturePlayer::GetMediaName() const
{
	FScopeLock Lock(&CriticalSection);
	if(MediaCaptureHelper != nil)
	{
		return FText::FromString(FString([MediaCaptureHelper getCaptureDeviceName]));
	}
	return FText();
}

IMediaView& FAvfMediaCapturePlayer::GetView()
{
	return *this;
}

void FAvfMediaCapturePlayer::Close()
{
	SCOPED_AUTORELEASE_POOL;
	
	FScopeLock Lock(&CriticalSection);
	
	CurrentRate = 0.f;
	CurrentTime = FTimespan::MinValue();
	URL = FString();
	
#if PLATFORM_MAC && WITH_EDITOR
	ThrottleDuration = 0.0;
	LastConsumedTimeStamp = 0.0;
#endif
	
	if(MediaCaptureHelper != nil)
	{
		[MediaCaptureHelper release];
		MediaCaptureHelper = nil;
		
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaClosed);
	}
}

void FAvfMediaCapturePlayer::HandleAuthStatusError(EAvfMediaCaptureAuthStatus AuthStatus, AVMediaType MediaType)
{
	FString ErrorString;
	
	switch(AuthStatus)
	{
		case EAvfMediaCaptureAuthStatus::Authorized:
		case EAvfMediaCaptureAuthStatus::NotDetermined:
		{
			// NO ERROR
			break;
		}
		case EAvfMediaCaptureAuthStatus::MissingInfoPListEntry:
		{
			ErrorString = NSLOCTEXT("AvfMediaCapture", "AvfMediaCapture_DeviceFail_MissingInfoPListEntry", "Cannot start capture: Missing Info.plist entry: ").ToString() + (MediaType == AVMediaTypeVideo ? TEXT("\"NSCameraUsageDescription\"") : TEXT("\"NSMicrophoneUsageDescription\""));
			break;
		}
		case EAvfMediaCaptureAuthStatus::Restricted:
		{
			ErrorString = NSLOCTEXT("AvfMediaCapture", "AvfMediaCapture_DeviceFail_Restricted", "Cannot start capture: Restricted").ToString();
			break;
		}
		case EAvfMediaCaptureAuthStatus::Denied:
		{
			ErrorString = NSLOCTEXT("AvfMediaCapture", "AvfMediaCapture_DeviceFail_Denied", "Cannot start capture: Denied").ToString();
			break;
		}
		default:
		{
			ErrorString = NSLOCTEXT("AvfMediaCapture", "AvfMediaCapture_DeviceFail_Unknown", "Cannot start capture: Unknown reason").ToString();
			break;
		}
	}
	
	if(ErrorString.Len() > 0)
	{
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);

		ErrorString += FString(TEXT(" (")) + URL + FString(TEXT(")"));
		FText Msg = FText::FromString(ErrorString);
		FText Title = NSLOCTEXT("AvfMediaCapture", "AvfMediaCapture_DeviceFail", "Capture Device Start Failed");
		FMessageDialog::Open(EAppMsgType::Ok, Msg, Title);
	}
}

void FAvfMediaCapturePlayer::CreateCaptureSession(NSString* DeviceIDString, AVMediaType MediaType)
{
	SCOPED_AUTORELEASE_POOL;
	
	FScopeLock Lock(&CriticalSection);
	
	if(MediaCaptureHelper == nil)
	{
		// Assuming only one Capture at the moment
		MediaCaptureHelper = [[AvfMediaCaptureHelper alloc] init: MediaType];
		BOOL bResult = [MediaCaptureHelper setupCaptureSession: DeviceIDString sampleBufferCallback: ^(CMSampleBufferRef SampleBuffer)
		{
			this->NewSampleBufferAvailable(SampleBuffer);
		}
		notificationCallback:^(NSNotification* Notification)
		{
			this->CaptureSystemNotification(Notification);
		}];
		
		if(bResult)
		{
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
			EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
		}
		else
		{
			Close();
		}
	}
}

bool FAvfMediaCapturePlayer::Open(const FString& Url, const IMediaOptions* Options)
{
	SCOPED_AUTORELEASE_POOL;
	
	Close();
	
	FScopeLock Lock(&CriticalSection);
	
	FString DeviceType;
	FString DeviceID;
	
	Url.Split(TEXT("://"), &DeviceType, &DeviceID);
	
	if((DeviceType.StartsWith(TEXT("vidcap")) || DeviceType.StartsWith(TEXT("audcap"))) && DeviceID.Len() > 0)
	{
		URL = Url;
		AVMediaType MediaType = DeviceType.StartsWith(TEXT("vidcap")) ? AVMediaTypeVideo : AVMediaTypeAudio;
		
		NSString* nsDeviceID = DeviceID.GetNSString();
		
		EAvfMediaCaptureAuthStatus CaptureAuthStatus = [AvfMediaCaptureHelper authorizationStatusForMediaType:MediaType];
		switch(CaptureAuthStatus)
		{
			case EAvfMediaCaptureAuthStatus::Authorized:
			{
				CreateCaptureSession(nsDeviceID, MediaType);
				break;
			}
			case EAvfMediaCaptureAuthStatus::NotDetermined:
			{
				TWeakPtr<FAvfMediaCapturePlayer, ESPMode::ThreadSafe> WeakPtr = this->AsWeak();
				[AvfMediaCaptureHelper requestAccessForMediaType:MediaType completionCallback:^(EAvfMediaCaptureAuthStatus ResultStatus)
				{
					TSharedPtr<FAvfMediaCapturePlayer, ESPMode::ThreadSafe> StrongPtr = WeakPtr.Pin();
					if(StrongPtr.IsValid())
					{
						if(ResultStatus == EAvfMediaCaptureAuthStatus::Authorized)
						{
							StrongPtr->CreateCaptureSession(nsDeviceID, MediaType);
						}
						else
						{
							StrongPtr->HandleAuthStatusError(ResultStatus, MediaType);
						}
					}
				}];
				break;
			}
			case EAvfMediaCaptureAuthStatus::MissingInfoPListEntry:
			case EAvfMediaCaptureAuthStatus::Restricted:
			case EAvfMediaCaptureAuthStatus::Denied:
			default:
			{
				HandleAuthStatusError(CaptureAuthStatus, MediaType);
				break;
			}
		}
		
		return MediaCaptureHelper != nil || CaptureAuthStatus == EAvfMediaCaptureAuthStatus::NotDetermined;
	}
	
	return false;
}

bool FAvfMediaCapturePlayer::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options)
{
	// Does not support open/playback from an Unreal Engine Archive
	return false;
}

bool FAvfMediaCapturePlayer::GetPlayerFeatureFlag(EFeatureFlag Flag) const
{
	return Flag == EFeatureFlag::PlayerSelectsDefaultTracks ? true : false;
}

// IMediaTracks Interface

bool FAvfMediaCapturePlayer::GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	SCOPED_AUTORELEASE_POOL;
	
	FScopeLock Lock(&CriticalSection);
	if(TrackIndex == 0 && FormatIndex >= 0)
	{
		NSArray<AVCaptureDeviceFormat*>* deviceFormats = [MediaCaptureHelper getCaptureDeviceAvailableFormats];
		
		if(FormatIndex < deviceFormats.count)
		{
			AVCaptureDeviceFormat* availableFormat = deviceFormats[FormatIndex];
			
			if(CMFormatDescriptionGetMediaType(availableFormat.formatDescription) == kCMMediaType_Audio)
			{
				CMAudioFormatDescriptionRef AudioFormatDescription = (CMAudioFormatDescriptionRef)availableFormat.formatDescription;
				AudioStreamBasicDescription const* ASBD = CMAudioFormatDescriptionGetStreamBasicDescription(AudioFormatDescription);

				OutFormat.BitsPerSample = ASBD->mBitsPerChannel;
				OutFormat.NumChannels = ASBD->mChannelsPerFrame;
				OutFormat.SampleRate = (uint32)ASBD->mSampleRate;
				OutFormat.TypeName = FStringFromAppleFourCharWordCode(ASBD->mFormatID);
				
				if((ASBD->mFormatFlags & kAudioFormatFlagIsNonMixable) != 0)
				{
					OutFormat.TypeName += TEXT(" (Non-Mixable)");
				}
			
				return true;
			}
		}
	}
	return false;
}

int32 FAvfMediaCapturePlayer::GetNumTracks(EMediaTrackType TrackType) const
{
	SCOPED_AUTORELEASE_POOL;
	
	FScopeLock Lock(&CriticalSection);
	if(MediaCaptureHelper != nil)
	{
		AVMediaType MediaType = [MediaCaptureHelper getCaptureDeviceMediaType];
		if	(	(TrackType == EMediaTrackType::Video && MediaType == AVMediaTypeVideo) ||
				(TrackType == EMediaTrackType::Audio && MediaType == AVMediaTypeAudio)
			)
		{
			return 1;
		}
	}
	return 0;
}

int32 FAvfMediaCapturePlayer::GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const
{
	SCOPED_AUTORELEASE_POOL;
	
	if(TrackIndex == 0)
	{
		FScopeLock Lock(&CriticalSection);
		if(MediaCaptureHelper != nil)
		{
			AVMediaType MediaType;
			switch (TrackType) {
			case EMediaTrackType::Audio:
				MediaType = AVMediaTypeAudio;
				break;
			case EMediaTrackType::Video:
				MediaType = AVMediaTypeVideo;
				break;
			default:
				return 0;
			}

			if ([[MediaCaptureHelper getCaptureDeviceMediaType] isEqualToString: MediaType])
			{
				return [[MediaCaptureHelper getCaptureDeviceAvailableFormats] count];
			}
			else
			{
				return 0;
			}
		}
	}
	
	return 0;
}

int32 FAvfMediaCapturePlayer::GetSelectedTrack(EMediaTrackType TrackType) const
{
	SCOPED_AUTORELEASE_POOL;
	
	FScopeLock Lock(&CriticalSection);
	if (MediaCaptureHelper != nil)
	{
		AVMediaType MediaType = [MediaCaptureHelper getCaptureDeviceMediaType];
		if	(	(TrackType == EMediaTrackType::Video && MediaType == AVMediaTypeVideo) ||
				(TrackType == EMediaTrackType::Audio && MediaType == AVMediaTypeAudio)
			)
		{
			return 0;
		}
	}
	return INDEX_NONE;
}

FText FAvfMediaCapturePlayer::GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	return FText();
}

int32 FAvfMediaCapturePlayer::GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const
{
	SCOPED_AUTORELEASE_POOL;
	
	if (TrackIndex == 0)
	{
		FScopeLock Lock(&CriticalSection);
		if (MediaCaptureHelper != nil)
		{
			AVMediaType MediaType = [MediaCaptureHelper getCaptureDeviceMediaType];
			if	(	(TrackType == EMediaTrackType::Video && MediaType == AVMediaTypeVideo) ||
					(TrackType == EMediaTrackType::Audio && MediaType == AVMediaTypeAudio)
				)
			{
				return [MediaCaptureHelper getCaptureDeviceActiveFormatIndex];
			}
		}
	}
	return INDEX_NONE;
}

FString FAvfMediaCapturePlayer::GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const
{
	return FString();
}

FString FAvfMediaCapturePlayer::GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	return FString();
}

bool FAvfMediaCapturePlayer::GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	SCOPED_AUTORELEASE_POOL;
	
	FScopeLock Lock(&CriticalSection);
	
	if (TrackIndex == 0 && FormatIndex >= 0)
	{
		NSArray<AVCaptureDeviceFormat*>* deviceFormats = [MediaCaptureHelper getCaptureDeviceAvailableFormats];
		if (FormatIndex < deviceFormats.count)
		{
			AVCaptureDeviceFormat* availableFormat = deviceFormats[FormatIndex];
			if (CMFormatDescriptionGetMediaType(availableFormat.formatDescription) == kCMMediaType_Video)
			{
				float fMinRate = FLT_MAX;
				float fMaxRate = FLT_MIN;
				
				NSArray<AVFrameRateRange*>* rateRanges = availableFormat.videoSupportedFrameRateRanges;
				for(uint32 i = 0;i < rateRanges.count;++i)
				{
					AVFrameRateRange* range = rateRanges[i];
					if (range.maxFrameRate > fMaxRate)
					{
						fMaxRate = range.maxFrameRate;
					}
					if (range.minFrameRate < fMinRate)
					{
						fMinRate = range.minFrameRate;
					}
				}
				
				// Missing rate info - set a sensible value
				if (fMinRate > fMaxRate)
				{
					fMinRate = fMaxRate = 30.f;
				}
				
				CMVideoDimensions Dimensions = CMVideoFormatDescriptionGetDimensions(availableFormat.formatDescription);
				
				OutFormat.Dim = FIntPoint(Dimensions.width, Dimensions.height);
				OutFormat.FrameRate = fMaxRate;
				OutFormat.FrameRates = TRange<float>(fMinRate, fMaxRate);
				
				FourCharCode subTypeCode = CMFormatDescriptionGetMediaSubType(availableFormat.formatDescription);
				OutFormat.TypeName = FStringFromAppleFourCharWordCode(subTypeCode);
				
				return true;
			}
		}
	}
	return false;
}

bool FAvfMediaCapturePlayer::SelectTrack(EMediaTrackType TrackType, int32 TrackIndex)
{
	return TrackIndex == 0;
}

bool FAvfMediaCapturePlayer::SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex)
{
	SCOPED_AUTORELEASE_POOL;
	
	if (TrackIndex == 0)
	{
		FScopeLock Lock(&CriticalSection);
		if (TrackType == EMediaTrackType::Video || TrackType == EMediaTrackType::Audio)
		{
			if ([MediaCaptureHelper setCaptureDeviceActiveFormatIndex:FormatIndex])
			{
				return true;
			}
		}
	}
	return false;
}

// IMediaControls Interface

bool FAvfMediaCapturePlayer::CanControl(EMediaControl Control) const
{
	return false;
}

FTimespan FAvfMediaCapturePlayer::GetDuration() const
{
	return FTimespan::MaxValue();
}

float FAvfMediaCapturePlayer::GetRate() const
{
	FScopeLock Lock(&CriticalSection);
	return CurrentRate;
}

EMediaState FAvfMediaCapturePlayer::GetState() const
{
	SCOPED_AUTORELEASE_POOL;
	
	FScopeLock Lock(&CriticalSection);
	if (MediaCaptureHelper != nil)
	{
		if (CurrentRate == 0.f && ![MediaCaptureHelper isCaptureRunning])
		{
			return EMediaState::Paused;
		}
		else
		{
			return EMediaState::Playing;
		}
	}
	return EMediaState::Closed;
}

EMediaStatus FAvfMediaCapturePlayer::GetStatus() const
{
	return EMediaStatus::None;
}

TRangeSet<float> FAvfMediaCapturePlayer::GetSupportedRates(EMediaRateThinning Thinning) const
{
	TRangeSet<float> Result;

	Result.Add(TRange<float>{0.0f});
	Result.Add(TRange<float>{1.0f});

	return Result;
}

FTimespan FAvfMediaCapturePlayer::GetTime() const
{
	FScopeLock Lock(&CriticalSection);
	return CurrentTime;
}

bool FAvfMediaCapturePlayer::IsLooping() const
{
	return false;
}

bool FAvfMediaCapturePlayer::Seek(const FTimespan& Time)
{
	return false;
}

bool FAvfMediaCapturePlayer::SetLooping(bool Looping)
{
	return false;
}

bool FAvfMediaCapturePlayer::SetRate(float Rate)
{
	SCOPED_AUTORELEASE_POOL
	
	FScopeLock Lock(&CriticalSection);
	if (Rate != CurrentRate)
	{
		if (Rate == 0.f && CurrentRate != 0.f)
		{
			[MediaCaptureHelper stopCaptureSession];
		}
		else if (Rate != 0.f && CurrentRate == 0.f)
		{
			[MediaCaptureHelper startCaptureSession];
		}
		
		CurrentRate = Rate;
	}
	return true;
}

void FAvfMediaCapturePlayer::CaptureSystemNotification(NSNotification* Notification)
{
	NSString* NotificationName = Notification.name;
	if (NotificationName == AVCaptureSessionRuntimeErrorNotification)
	{
		Close();
	}
	else if (NotificationName == AVCaptureSessionDidStartRunningNotification)
	{
		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);
	}
	else if (NotificationName == AVCaptureSessionDidStopRunningNotification)
	{
		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);
	}
	else if (NotificationName == AVCaptureSessionWasInterruptedNotification)
	{
		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);
	}
	else if (NotificationName == AVCaptureSessionInterruptionEndedNotification)
	{
		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);
	}
#if PLATFORM_MAC && WITH_EDITOR
	else if (NotificationName == NSApplicationDidBecomeActiveNotification)
	{
		ThrottleDuration = 0.0;
	}
	else if (NotificationName == NSApplicationWillResignActiveNotification)
	{
		ThrottleDuration = 1.0;
	}
#endif
}

void FAvfMediaCapturePlayer::NewSampleBufferAvailable(CMSampleBufferRef SampleBuffer)
{
	if (!CMSampleBufferIsValid(SampleBuffer) || !CMSampleBufferDataIsReady(SampleBuffer))
	{
		return;
	}
	
	FScopeLock Lock(&CriticalSection);
	
	if (CurrentRate == 0.f)
	{
		return;
	}

	// Allow each media type to compute the duration as CMSampleBuffer doesn't always report valid timings
	CMFormatDescriptionRef FormatDescription = CMSampleBufferGetFormatDescription(SampleBuffer);
	if (FormatDescription != NULL)
	{
		CMMediaType MediaType = CMFormatDescriptionGetMediaType(FormatDescription);
		if (MediaType == kCMMediaType_Audio)
		{
			ProcessSampleBufferAudio(SampleBuffer);
		}
		else if (MediaType == kCMMediaType_Video)
		{
			ProcessSampleBufferVideo(SampleBuffer);
		}
	}
}

FTimespan FAvfMediaCapturePlayer::UpdateInternalTime(CMSampleBufferRef SampleBuffer, FTimespan const& ComputedBufferDuration)
{
	FTimespan SampleDuration(0);
		
	{
		CMTime SampleBufferDuration = CMSampleBufferGetDuration(SampleBuffer);
		if ((SampleBufferDuration.flags & kCMTimeFlags_Valid) == kCMTimeFlags_Valid)
		{
			SampleDuration = FTimespan::FromSeconds(CMTimeGetSeconds(SampleBufferDuration));
		}
	}
	
	if (SampleDuration.IsZero())
	{
		SampleDuration = ComputedBufferDuration;
	}
	
	if (CurrentTime == FTimespan::MinValue())
	{
		CurrentTime = FTimespan::Zero();
	}
	else if (CurrentTime >= FTimespan::Zero())
	{
		CurrentTime += SampleDuration;
	}
	
	return SampleDuration;
}

void FAvfMediaCapturePlayer::ProcessSampleBufferVideo(CMSampleBufferRef SampleBuffer)
{
	check(SampleBuffer);
	
	FTimespan SampleDuration = UpdateInternalTime(SampleBuffer, FTimespan::FromSeconds(1.0 / 60.0));
	
#if PLATFORM_MAC && WITH_EDITOR
	// Throttle frames pasted to the editor when not in focus
	double PlatformTime = FPlatformTime::Seconds();
	if (ThrottleDuration > 0.0 && (PlatformTime - LastConsumedTimeStamp) < ThrottleDuration)
	{
		return;
	}
	LastConsumedTimeStamp = PlatformTime;
#endif

	CVPixelBufferRef PixelBuffer = CMSampleBufferGetImageBuffer(SampleBuffer);
	if (PixelBuffer)
	{
		check(CMSampleBufferGetNumSamples(SampleBuffer) == 1);
		check(!CVPixelBufferIsPlanar(PixelBuffer));
		
		if (!MetalTextureCache)
		{
            id<MTLDevice> Device = (__bridge id<MTLDevice>)GetIMetalDynamicRHI()->RHIGetDevice();
			check(Device);
			
			CVReturn Return = CVMetalTextureCacheCreate(kCFAllocatorDefault, nullptr, Device, nullptr, &MetalTextureCache);
			check(Return == kCVReturnSuccess);
		}

		int32 Width = CVPixelBufferGetWidth(PixelBuffer);
		int32 Height = CVPixelBufferGetHeight(PixelBuffer);
		
		CVMetalTextureRef TextureRef = nullptr;
		CVReturn Result = CVMetalTextureCacheCreateTextureFromImage(kCFAllocatorDefault, MetalTextureCache, PixelBuffer, nullptr, MTLPixelFormatBGRA8Unorm_sRGB, Width, Height, 0, &TextureRef);
		check(Result == kCVReturnSuccess);
		check(TextureRef);

		FIntPoint Dim(Width, Height);

		// Retain the pixel buffer so it survives past this callback. The render command
		// will pass it to FAvfMediaTextureSample which takes its own retain, then we release ours.
		CFRetain(PixelBuffer);

		ENQUEUE_RENDER_COMMAND(FAvfMediaCapturePlayer_ProcessSampleBufferVideo_CreateTexture)(
            [WeakPlayer = this->AsWeak(), Dim, SampleDuration, TextureRef, PixelBuffer](FRHICommandListImmediate& RHICmdList)
            {
            	if (const TSharedPtr<FAvfMediaCapturePlayer> Player = WeakPlayer.Pin())
            	{
					IMetalDynamicRHI* MetalDynamicRHI = GetIMetalDynamicRHI();
										
					TRefCountPtr<FRHITexture> ShaderResource = MetalDynamicRHI->RHICreateTexture2DFromCVMetalTexture(PF_B8G8R8A8, ETextureCreateFlags::NoTiling | ETextureCreateFlags::ShaderResource, FClearValueBinding::White, TextureRef);
					CFRelease(TextureRef);
					
					TSharedRef<FAvfMediaTextureSample, ESPMode::ThreadSafe> VideoSample = Player->VideoSamplePool.AcquireShared();
					VideoSample->Initialize(ShaderResource, Dim, Dim, Player->CurrentTime, SampleDuration, PixelBuffer);
					CFRelease(PixelBuffer);
            
					Player->MediaSamples->AddVideo(VideoSample);
            	}
            	else
            	{
            		// Player was destroyed — still need to release our retains
            		CFRelease(TextureRef);
            		CFRelease(PixelBuffer);
            	}
        });
	}
}

void FAvfMediaCapturePlayer::ProcessSampleBufferAudio(CMSampleBufferRef SampleBuffer)
{
	check(SampleBuffer);
	
	// Avoid using CMSampleBufferGetAudioBufferListWithRetainedBlockBuffer as that appears to copy data - don't want to incurr the extra copy before we even copy out into the UE structure
	CMFormatDescriptionRef FormatDescription = CMSampleBufferGetFormatDescription(SampleBuffer);
	if (FormatDescription != NULL && CMFormatDescriptionGetMediaType(FormatDescription) == kCMMediaType_Audio)
	{
		CMAudioFormatDescriptionRef AudioFormatDescription = (CMAudioFormatDescriptionRef)FormatDescription;
		AudioStreamBasicDescription const* ASBD = CMAudioFormatDescriptionGetStreamBasicDescription(AudioFormatDescription);
		check(ASBD);
		
		CMBlockBufferRef BlockBuffer = CMSampleBufferGetDataBuffer(SampleBuffer);
		if (BlockBuffer != NULL)
		{
			const size_t BufferSize = CMBlockBufferGetDataLength(BlockBuffer);
			const size_t BitDepth = ASBD->mBitsPerChannel;
			const size_t NumChannels = ASBD->mChannelsPerFrame;
			
			// CMSampleBufferGetNumSamples() is per channel so this is really the frame count
			const size_t NumFrames = CMSampleBufferGetNumSamples(SampleBuffer);
			
			// It should match a computed value - if it doesn't then use this computed value instead as Apple switch meaning of Frames and Samples depending on doc / API.
			check(NumFrames == BufferSize / NumChannels / (BitDepth / 8));
			
			// Get valid engine format
			EMediaAudioSampleFormat SampleFormat = EMediaAudioSampleFormat::Undefined;
			if ((ASBD->mFormatFlags & kLinearPCMFormatFlagIsFloat) != 0)
			{
				if (BitDepth == 32)		SampleFormat = EMediaAudioSampleFormat::Float;
				else if (BitDepth == 64)	SampleFormat = EMediaAudioSampleFormat::Double;
			}
			else
			{
				if (BitDepth == 32)		SampleFormat = EMediaAudioSampleFormat::Int32;
				else if (BitDepth == 24) SampleFormat = EMediaAudioSampleFormat::Undefined; // 24-bit int is not supported by UE
				else if (BitDepth == 16)	SampleFormat = EMediaAudioSampleFormat::Int16;
				else if (BitDepth == 8)	SampleFormat = EMediaAudioSampleFormat::Int8;
			}

			FTimespan SampleDuration = UpdateInternalTime(SampleBuffer, FTimespan::FromSeconds(((double)NumFrames) / ASBD->mSampleRate));
			
			if (SampleFormat != EMediaAudioSampleFormat::Undefined)
			{
				TSharedRef<FAvfMediaAudioSample, ESPMode::ThreadSafe> AudioSample = AudioSamplePool.AcquireShared();
				if (AudioSample->Initialize(BufferSize,
											NumFrames,
											NumChannels,
											ASBD->mSampleRate,
											CurrentTime,
											SampleDuration,
											SampleFormat))
				{
					size_t CurrentOffset = 0;
					uint8* DestData = AudioSample->GetMutableBuffer();

					while(CurrentOffset < BufferSize)
					{
						char* SrcData = NULL;

						size_t LengthAtOffset = 0;
						size_t BlockTotalSize = 0;

						if (CMBlockBufferGetDataPointer(BlockBuffer, CurrentOffset, &LengthAtOffset, &BlockTotalSize, &SrcData) != noErr)
						{
							break; // can't get data pointer force exit while loop
						}

						if (LengthAtOffset == 0)
						{
							break; // no data force exit while loop
						}

						checkf((CurrentOffset + LengthAtOffset) <= BufferSize, TEXT("Buffer overflow"));
						checkf(BlockTotalSize == BufferSize, TEXT("Total CMBlockBuffer size does not match data pointer total sizes"));
						checkf(CMBlockBufferIsRangeContiguous(BlockBuffer, CurrentOffset, LengthAtOffset), TEXT("Returned data pointer for CMBlockBuffer range is not contiguous"));

						if (SrcData != NULL)
						{
							if ((ASBD->mFormatFlags & kAudioFormatFlagIsNonInterleaved) == 0 || NumChannels == 1)
							{
								// Interleaved data OR single channel - direct copy
								FMemory::Memcpy(DestData + CurrentOffset, SrcData, LengthAtOffset);
							}
							else
							{
								// Non Interleaved AND multiple channel - copy to interleaved buffer
								size_t SrcNumFrames = LengthAtOffset / (BitDepth / 8);
								size_t SrcFrameOffset = CurrentOffset / (BitDepth / 8);
								
								if (SampleFormat == EMediaAudioSampleFormat::Float) 			InterleaveCopy<float>(DestData, SrcData, NumChannels, NumFrames, SrcFrameOffset, SrcNumFrames);
								else if (SampleFormat == EMediaAudioSampleFormat::Double) 	InterleaveCopy<double>(DestData, SrcData, NumChannels, NumFrames, SrcFrameOffset, SrcNumFrames);
								else if (SampleFormat == EMediaAudioSampleFormat::Int32) 	InterleaveCopy<int32>(DestData, SrcData, NumChannels, NumFrames, SrcFrameOffset, SrcNumFrames);
								else if (SampleFormat == EMediaAudioSampleFormat::Int16) 	InterleaveCopy<int16>(DestData, SrcData, NumChannels, NumFrames, SrcFrameOffset, SrcNumFrames);
								else if (SampleFormat == EMediaAudioSampleFormat::Int8) 		InterleaveCopy<int8>(DestData, SrcData, NumChannels, NumFrames, SrcFrameOffset, SrcNumFrames);
								else break;	// unsupported force exit while loop
							}
						}

						CurrentOffset += LengthAtOffset;
					}
					
					if (CurrentOffset < BufferSize)
					{
						// Error case - clear the output buffer to avoid unwanted noise
						FMemory::Memset(DestData, 0, BufferSize);
					}

					MediaSamples->AddAudio(AudioSample);
				}
			}
		}
	}
}
