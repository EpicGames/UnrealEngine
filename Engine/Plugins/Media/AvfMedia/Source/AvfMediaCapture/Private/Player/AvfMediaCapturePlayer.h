// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaCache.h"
#include "IMediaControls.h"
#include "IMediaPlayer.h"
#include "IMediaTracks.h"
#include "IMediaView.h"
#include "IMediaTextureSample.h"
#include "IMediaAudioSample.h"
#include "IMediaEventSink.h"
#include "Misc/Timespan.h"

#include "AvfMediaCaptureHelper.h"
#include "AvfMediaAudioSample.h"
#include "AvfMediaTextureSample.h"

class FMediaSamples;
class FAvfMediaAudioSamplePool;
class FAvfMediaTextureSamplePool;

class FAvfMediaCapturePlayer
	: public IMediaPlayer
	, public IMediaTracks
	, protected IMediaCache
	, protected IMediaControls
	, protected IMediaView
	, public TSharedFromThis<FAvfMediaCapturePlayer>
{
public:
	
	FAvfMediaCapturePlayer(IMediaEventSink& InEventSink);
	virtual ~FAvfMediaCapturePlayer();

public:
	// IMediaPlayer Interface
	virtual void Close() override;
	virtual IMediaCache& GetCache() override;
	virtual IMediaControls& GetControls() override;
	virtual FString GetInfo() const override;
	virtual FGuid GetPlayerPluginGUID() const override;
	virtual IMediaSamples& GetSamples() override;
	virtual FString GetStats() const override;
	virtual IMediaTracks& GetTracks() override;
	virtual FString GetUrl() const override;
	virtual IMediaView& GetView() override;
	virtual bool Open(const FString& Url, const IMediaOptions* Options) override;
	virtual bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options) override;
	virtual FText GetMediaName() const override;
	virtual bool GetPlayerFeatureFlag(EFeatureFlag Flag) const override;
	
	// IMediaTracks Interface
	virtual bool GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const override;
	virtual int32 GetNumTracks(EMediaTrackType TrackType) const override;
	virtual int32 GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetSelectedTrack(EMediaTrackType TrackType) const override;
	virtual FText GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual bool GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const override;
	virtual bool SelectTrack(EMediaTrackType TrackType, int32 TrackIndex) override;
	virtual bool SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex) override;

protected:
	
	// IMediaControls Interface
	virtual bool CanControl(EMediaControl Control) const override;
	virtual FTimespan GetDuration() const override;
	virtual float GetRate() const override;
	virtual EMediaState GetState() const override;
	virtual EMediaStatus GetStatus() const override;
	virtual TRangeSet<float> GetSupportedRates(EMediaRateThinning Thinning) const override;
	virtual FTimespan GetTime() const override;
	virtual bool IsLooping() const override;
	virtual bool Seek(const FTimespan& Time) override;
	virtual bool SetLooping(bool Looping) override;
	virtual bool SetRate(float Rate) override;
	
private:

	void CreateCaptureSession(NSString* deviceIDString, AVMediaType MediaType);
	void CaptureSystemNotification(NSNotification* Notification);
	void HandleAuthStatusError(EAvfMediaCaptureAuthStatus AuthStatus, AVMediaType MediaType);

	FTimespan UpdateInternalTime(CMSampleBufferRef SampleBuffer, FTimespan const& ComputedBufferDuration);
	
	void NewSampleBufferAvailable(CMSampleBufferRef SampleBuffer);
	void ProcessSampleBufferVideo(CMSampleBufferRef SampleBuffer);
	void ProcessSampleBufferAudio(CMSampleBufferRef SampleBuffer);

private:

	/** The media event handler. */
	IMediaEventSink& EventSink;

	/** Media sample collection. */
	TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> MediaSamples;
	
	/** Media capture AV Obj-C Helper */
	AvfMediaCaptureHelper* MediaCaptureHelper;
	
	/** Reuse Sample Pool Objects */
	FAvfMediaAudioSamplePool AudioSamplePool;
	FAvfMediaTextureSamplePool VideoSamplePool;
	
	/** Internal CoreVideo Metal Texture cache to handle AVFoundation resource pooling and optimisations */
	CVMetalTextureCacheRef MetalTextureCache;
	
	/** Media Playback Info and Control data */
	float CurrentRate;
	FTimespan CurrentTime;
	FString URL;

#if PLATFORM_MAC && WITH_EDITOR
	double ThrottleDuration;
	double LastConsumedTimeStamp;
#endif

	mutable FCriticalSection CriticalSection;
};
