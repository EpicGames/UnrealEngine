// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvfFileMediaSourceFactory.h"

#include "Misc/Paths.h"
#include "FileMediaSource.h"
#include "Containers/UnrealString.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AvfFileMediaSourceFactory)


/* UAvfFileMediaSourceFactory structors
 *****************************************************************************/

UAvfFileMediaSourceFactory::UAvfFileMediaSourceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Formats.Add(TEXT("3g2;3G2 Multimedia Stream"));
	Formats.Add(TEXT("3gp;3GP Video Stream"));
	Formats.Add(TEXT("3gp2;3GPP2 Multimedia File"));
	Formats.Add(TEXT("3gpp;3GPP Multimedia File"));
	Formats.Add(TEXT("ac3;AC-3 Audio File"));
	Formats.Add(TEXT("amr;Adaptive Multi-Rate Audio"));
	Formats.Add(TEXT("au;Audio File"));
	Formats.Add(TEXT("bwf;Broadcast Wave Audio"));
	Formats.Add(TEXT("caf;Core Audio"));
	Formats.Add(TEXT("cdda;Compact Disc Digital Audio"));
	Formats.Add(TEXT("m4a;Apple MPEG-4 Audio"));
	Formats.Add(TEXT("m4v;Apple MPEG-4 Video"));
	Formats.Add(TEXT("mov;Apple QuickTime Movie"));
	Formats.Add(TEXT("mp4;MPEG-4 Movie"));
	Formats.Add(TEXT("sdv;Samsung Digital Video"));
	Formats.Add(TEXT("snd;Sound File"));

	SupportedClass = UFileMediaSource::StaticClass();
	bEditorImport = true;
}


/* UFactory overrides
 *****************************************************************************/

bool UAvfFileMediaSourceFactory::FactoryCanImport(const FString& Filename)
{
	return true;
}


UObject* UAvfFileMediaSourceFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	UFileMediaSource* MediaSource = NewObject<UFileMediaSource>(InParent, InClass, InName, Flags);
	MediaSource->SetFilePath(CurrentFilename);

	return MediaSource;
}

