// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"

#include "AvfFileMediaSourceFactory.generated.h"


/**
 * Implements a factory for UFileMediaSource objects.
 */
UCLASS(hidecategories=Object)
class UAvfFileMediaSourceFactory
	: public UFactory
{
	GENERATED_UCLASS_BODY()

public:

	//~ UFactory Interface

	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
};
