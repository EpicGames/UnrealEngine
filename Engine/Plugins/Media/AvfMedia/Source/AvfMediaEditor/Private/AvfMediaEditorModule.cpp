// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


/**
 * Implements the AvfMediaEditor module.
 */
class FAvfMediaEditorModule
	: public IModuleInterface
{
public:

	//~ IModuleInterface interface

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }
};


IMPLEMENT_MODULE(FAvfMediaEditorModule, AvfMediaEditor);
