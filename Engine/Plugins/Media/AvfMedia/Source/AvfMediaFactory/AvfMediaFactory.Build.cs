// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AvfMediaFactory : ModuleRules
	{
		public AvfMediaFactory(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MediaAssets",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"AvfMedia",
					"Media",
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
				});

			if (Target.Type == TargetType.Editor)
			{
				DynamicallyLoadedModuleNames.Add("Settings");
				PrivateIncludePathModuleNames.Add("Settings");
			}

			if ((Target.Platform == UnrealTargetPlatform.IOS) ||
				(Target.Platform == UnrealTargetPlatform.Mac))
			{
				DynamicallyLoadedModuleNames.Add("AvfMedia");
			}
		}
	}
}
