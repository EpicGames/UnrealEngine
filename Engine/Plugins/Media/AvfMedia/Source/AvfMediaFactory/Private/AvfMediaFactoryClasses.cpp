// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvfMediaSettings.h"


UAvfMediaSettings::UAvfMediaSettings()
	: NativeAudioOut(false)
{ }
