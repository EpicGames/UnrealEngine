// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvfMediaFactoryPrivate.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "IMediaModule.h"
#include "IMediaPlayerFactory.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"

#if WITH_EDITOR
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
	#include "UObject/Class.h"
	#include "UObject/WeakObjectPtr.h"
#endif

#include "AvfMediaSettings.h"

#include "IAvfMediaModule.h"


DEFINE_LOG_CATEGORY(LogAvfMediaFactory);

#define LOCTEXT_NAMESPACE "FAvfMediaFactoryModule"


/**
 * Implements the AvfMediaFactory module.
 */
class FAvfMediaFactoryModule
	: public IMediaPlayerFactory
	, public IModuleInterface
{
public:

	//~ IMediaPlayerFactory interface

	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
	}

	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		FString Scheme;
		FString Location;

		// check scheme
		if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
			}

			return 0;
		}

		if (!SupportedUriSchemes.Contains(Scheme))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
			}

			return 0;
		}

		// check file extension
		if (Scheme == TEXT("file"))
		{
			const FString Extension = FPaths::GetExtension(Location, false);

			if (!SupportedFileExtensions.Contains(Extension))
			{
				if (OutErrors != nullptr)
				{
					OutErrors->Add(FText::Format(LOCTEXT("ExtensionNotSupported", "The file extension '{0}' is not supported"), FText::FromString(Extension)));
				}

				return 0;
			}
		}

		return 80;
	}

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		auto AvfMediaModule = FModuleManager::LoadModulePtr<IAvfMediaModule>("AvfMedia");
		return (AvfMediaModule != nullptr) ? AvfMediaModule->CreatePlayer(EventSink) : nullptr;
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("MediaPlayerDisplayName", "Apple AV Foundation");
	}

	virtual FName GetPlayerName() const override
	{
		static FName PlayerName(TEXT("AvfMedia"));
		return PlayerName;
	}

	virtual FGuid GetPlayerPluginGUID() const override
	{
		static FGuid PlayerPluginGUID(0xef660169, 0xa304472b, 0xb69e1281, 0xc1687654);
		return PlayerPluginGUID;
	}

	virtual const TArray<FString>& GetSupportedPlatforms() const override
	{
		return SupportedPlatforms;
	}

	virtual bool SupportsFeature(EMediaFeature Feature) const override
	{
		return ((Feature == EMediaFeature::AudioSamples) ||
				(Feature == EMediaFeature::AudioTracks) ||
				(Feature == EMediaFeature::CaptionTracks) ||
				(Feature == EMediaFeature::OverlaySamples) ||
				(Feature == EMediaFeature::VideoSamples) ||
				(Feature == EMediaFeature::VideoTracks));
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// supported schemes
		SupportedUriSchemes.Add(TEXT("file"));
		SupportedUriSchemes.Add(TEXT("http"));
		SupportedUriSchemes.Add(TEXT("https"));

		// supported platforms
		SupportedPlatforms.Add(TEXT("iOS"));
		SupportedPlatforms.Add(TEXT("Mac"));

		// supported file extensions
		SupportedFileExtensions.Add(TEXT("3g2"));
		SupportedFileExtensions.Add(TEXT("3gp"));
		SupportedFileExtensions.Add(TEXT("3gp2"));
		SupportedFileExtensions.Add(TEXT("3gpp"));
		SupportedFileExtensions.Add(TEXT("ac3"));
		SupportedFileExtensions.Add(TEXT("aif"));
		SupportedFileExtensions.Add(TEXT("aifc"));
		SupportedFileExtensions.Add(TEXT("aiff"));
		SupportedFileExtensions.Add(TEXT("amr"));
		SupportedFileExtensions.Add(TEXT("au"));
		SupportedFileExtensions.Add(TEXT("bwf"));
		SupportedFileExtensions.Add(TEXT("caf"));
		SupportedFileExtensions.Add(TEXT("cdda"));
		SupportedFileExtensions.Add(TEXT("m4a"));
		SupportedFileExtensions.Add(TEXT("m4v"));
		SupportedFileExtensions.Add(TEXT("mov"));
		SupportedFileExtensions.Add(TEXT("mp4"));
		SupportedFileExtensions.Add(TEXT("qt"));
		SupportedFileExtensions.Add(TEXT("sdv"));
		SupportedFileExtensions.Add(TEXT("snd"));
		SupportedFileExtensions.Add(TEXT("wav"));
		SupportedFileExtensions.Add(TEXT("wave"));

#if WITH_EDITOR
		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "AvfMedia",
				LOCTEXT("AvfMediaSettingsName", "AVF Media"),
				LOCTEXT("AvfMediaSettingsDescription", "Configure the AVF Media plug-in."),
				GetMutableDefault<UAvfMediaSettings>()
			);
		}
#endif //WITH_EDITOR

		// register player factory
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->RegisterPlayerFactory(*this);
		}
	}

	virtual void ShutdownModule() override
	{
		// unregister player factory
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterPlayerFactory(*this);
		}

#if WITH_EDITOR
		// unregister settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "AvfMedia");
		}
#endif //WITH_EDITOR
	}

private:

	/** List of supported media file types. */
	TArray<FString> SupportedFileExtensions;

	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAvfMediaFactoryModule, AvfMediaFactory);
