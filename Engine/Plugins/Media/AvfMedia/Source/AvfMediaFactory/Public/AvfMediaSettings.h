// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "AvfMediaSettings.generated.h"

#define UE_API AVFMEDIAFACTORY_API


/**
 * Settings for the AvfMedia plug-in.
 */
UCLASS(MinimalAPI, config=Engine)
class UAvfMediaSettings
	: public UObject
{
	GENERATED_BODY()

public:
	 
	/** Default constructor. */
	UE_API UAvfMediaSettings();

public:

	/** Play audio tracks via the operating system's native sound mixer. */
	UPROPERTY(config, EditAnywhere, Category=Debug)
	bool NativeAudioOut;
};

#undef UE_API
